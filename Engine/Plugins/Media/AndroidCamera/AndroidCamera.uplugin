{
	"FileVersion" : 3,
	"Version" : 2,
	"VersionName" : "2.0",
	"FriendlyName" : "Android Camera Player",
	"Description" : "Implements camera preview using the Android Camera library.",
	"Category" : "Media Players",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "https://forums.unrealengine.com/showthread.php?46879-Media-Framework-Documentation-for-4-5-Preview",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name" : "AndroidCamera",
			"Type" : "RuntimeNoCommandlet",
			"LoadingPhase" : "PreLoadingScreen",
			"PlatformAllowList" : [ "Android" ]
		},
		{
			"Name" : "AndroidCameraEditor",
			"Type" : "Editor",
			"LoadingPhase" : "PostEngineInit"
		},
		{
			"Name" : "AndroidCameraFactory",
			"Type" : "Editor",
			"LoadingPhase" : "PostEngineInit"
		},
		{
			"Name" : "AndroidCameraFactory",
			"Type" : "RuntimeNoCommandlet",
			"LoadingPhase" : "PostEngineInit",
			"PlatformAllowList" : [ "Android" ]
		}
	]
}