// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;

namespace UnrealBuildTool.Rules
{
	public class AndroidCamera : ModuleRules
	{
		public AndroidCamera(ReadOnlyTargetRules Target) : base(Target)
		{
			DynamicallyLoadedModuleNames.AddRange(
				new string[] {
					"Media",
				});

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"AndroidCameraFactory",
					"Core",
					"Engine",
					"Launch",
					"MediaUtils",
					"RenderCore",
				});

			PrivateIncludePathModuleNames.AddRange(
				new string[] {
					"Media",
				});

			string PluginPath = Utils.MakePathRelativeTo(ModuleDirectory, Target.RelativeEnginePath);
			AdditionalPropertiesForReceipt.Add("AndroidPlugin", Path.Combine(PluginPath, "AndroidCamera_UPL.xml"));
		}
	}
}
