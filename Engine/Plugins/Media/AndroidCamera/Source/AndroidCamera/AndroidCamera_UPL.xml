<?xml version="1.0" encoding="utf-8"?>
<!--AndroidCamera plugin additions-->
<root xmlns:android="http://schemas.android.com/apk/res/android">
	<!-- init section is always evaluated once per architecture -->
	<init>
		<log text="AndroidCamera init"/>
		<!-- currently supports all architectures -->
		<setBool result="bSupported" value="true"/>
		<if condition="bSupported">
			<true>
				<!-- see if hardware camera required -->
				<setBoolFromProperty result="bEnablePermission" ini="Engine" section="/Script/AndroidCameraEditor.AndroidCameraRuntimeSettings" property="bEnablePermission" default="true"/>
				<if condition="bEnablePermission">
					<true>
						<log text="Camera permission will be added in AndroidManifest.xml"/>
						<setBoolFromProperty result="bRequiresAnyCamera" ini="Engine" section="/Script/AndroidCameraEditor.AndroidCameraRuntimeSettings" property="bRequiresAnyCamera" default="false"/>
						<setBoolFromProperty result="bRequiresBackFacingCamera" ini="Engine" section="/Script/AndroidCameraEditor.AndroidCameraRuntimeSettings" property="bRequiresBackFacingCamera" default="false"/>
						<setBoolFromProperty result="bRequiresFrontFacingCamera" ini="Engine" section="/Script/AndroidCameraEditor.AndroidCameraRuntimeSettings" property="bRequiresFrontFacingCamera" default="false"/>
						<setBoolOr result="bRequiresFrontOrBackFacingCamera" arg1="$B(bRequiresBackFacingCamera)" arg2="$B(bRequiresFrontFacingCamera)"/>
					</true>
				</if>
			</true>
			<false>
				<log text="Camera not supported for this architecture, disabled."/>
			</false>
		</if>
	</init>

	<!-- optional updates applied to AndroidManifest.xml -->
	<androidManifestUpdates>
		<if condition="bSupported">
			<true>
				<if condition="bEnablePermission">
					<true>
						<addPermission android:name="android.permission.CAMERA"/>
						<if condition="bRequiresFrontOrBackFacingCamera">
							<false>
								<if condition="bRequiresAnyCamera">
									<true>
										<addFeature android:name="android.hardware.camera.any" android:required="true"/>
									</true>
									<false>
										<addFeature android:name="android.hardware.camera" android:required="false"/>
										<addFeature android:name="android.hardware.camera.front" android:required="false"/>
									</false>
								</if>
							</false>
							<true>
								<if condition="bRequiresBackFacingCamera">
									<true>
										<addFeature android:name="android.hardware.camera" android:required="true"/>
									</true>
									<false>
										<addFeature android:name="android.hardware.camera" android:required="false"/>
									</false>
								</if>
								<if condition="bRequiresFrontFacingCamera">
									<true>
										<addFeature android:name="android.hardware.camera.front" android:required="true"/>
									</true>
									<false>
										<addFeature android:name="android.hardware.camera.front" android:required="false"/>
									</false>
								</if>
							</true>
						</if>
					</true>
				</if>
			</true>
		</if>
	</androidManifestUpdates>
	
	<!-- optional additions to proguard -->
	<proguardAdditions>
		<insert>
-keep class com.epicgames.unreal.CameraPlayer14 {
   public *;
}
-keep class com.epicgames.unreal.CameraPlayer14$FrameUpdateInfo {
        public *;
}
-keep class com.epicgames.unreal.CameraPlayer14$AudioTrackInfo {
        public *;
}
-keep class com.epicgames.unreal.CameraPlayer14$CaptionTrackInfo {
        public *;
}
-keep class com.epicgames.unreal.CameraPlayer14$VideoTrackInfo {
        public *;
}
		</insert>
	</proguardAdditions>

	<!-- optional files or directories to copy to Intermediate/Android/APK -->
	<resourceCopies>
		<if condition="bSupported">
			<true>
				<copyDir src="$S(PluginDir)/Java" dst="$S(BuildDir)" />
			</true>
		</if>
	</resourceCopies>
	
	<!-- optional additions to the GameActivity imports in GameActivity.java -->
	<gameActivityImportAdditions>
		<if condition="bSupported">
			<true>
				<insert>
	import android.hardware.Camera;
				</insert>
			</true>
		</if>
	</gameActivityImportAdditions>
	
	<!-- optional additions to the GameActivity class in GameActivity.java -->
	<gameActivityClassAdditions>
		<if condition="bSupported">
			<true>
				<insert>
	private int AndroidCamera_Count = -1;

	public int AndroidThunkJava_AndroidCamera_CountCameras()
	{
		if (AndroidCamera_Count &gt;= 0)
		{
			return AndroidCamera_Count;
		}
		
		// check if camera supported
		if (!activityContext.getPackageManager().hasSystemFeature(PackageManager.FEATURE_CAMERA_ANY))
		{
			AndroidCamera_Count = 0;
			return AndroidCamera_Count;
		}

		// discover available cameras and remember front and back IDs if available
		AndroidCamera_Count = Camera.getNumberOfCameras();
		return AndroidCamera_Count;
	}
	
	public String AndroidThunkJava_AndroidCamera_GetCameraUrl(int index)
	{
		if (index &lt; 0 || index &gt; AndroidCamera_Count)
		{
			return null;
		}
		
		Camera.CameraInfo cameraInfo = new Camera.CameraInfo();
		Camera.getCameraInfo(index, cameraInfo);
		if (cameraInfo.facing == Camera.CameraInfo.CAMERA_FACING_FRONT)
		{
			return "vidcap://front";
		}
		if (cameraInfo.facing == Camera.CameraInfo.CAMERA_FACING_BACK)
		{
			return "vidcap://rear";
		}
		return "vidcap://" + index;
	}
				</insert>
			</true>
		</if>
	</gameActivityClassAdditions>
	
	<!-- optional additions to GameActivity onCreate in GameActivity.java
	<gameActivityOnCreateAdditions>
	</gameActivityOnCreateAdditions>
	-->
	
	<!-- optional additions to GameActivity onDestroy in GameActivity.java
	<gameActivityOnDestroyAdditions>
	</gameActivityOnDestroyAdditions>
	-->
	
	<!-- optional additions to GameActivity onStop in GameActivity.java -->
	<gameActivityOnStopAdditions>
		<if condition="bSupported">
			<true>
				<insert>
				</insert>
			</true>
		</if>
	</gameActivityOnStopAdditions>
	

	<!-- optional additions to GameActivity onPause in GameActivity.java -->
	<gameActivityOnPauseAdditions>
		<if condition="bSupported">
			<true>
				<insert>
				</insert>
			</true>
		</if>
	</gameActivityOnPauseAdditions>

	<!-- optional additions to GameActivity onResume in GameActivity.java -->
	<gameActivityOnResumeAdditions>
		<if condition="bSupported">
			<true>
				<insert>
				</insert>
			</true>
		</if>
	</gameActivityOnResumeAdditions>

	<!-- optional additions to GameActivity onActivityResult in GameActivity.java
	<gameActivityOnActivityResultAdditions>
	</gameActivityOnActivityResultAdditions>
	-->

	<!-- optional libraries to load in GameActivity.java before libUnreal.so -->
	<soLoadLibrary>
	</soLoadLibrary>
</root>
