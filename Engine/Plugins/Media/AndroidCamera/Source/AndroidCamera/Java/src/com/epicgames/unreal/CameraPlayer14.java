// Copyright Epic Games, Inc. All Rights Reserved.

package com.epicgames.unreal;

import android.Manifest;
import android.app.Activity;
import android.app.Application;
import android.content.Context;
import android.content.pm.PackageManager;
import android.hardware.camera2.params.StreamConfigurationMap;
import android.media.Image;
import android.media.ImageReader;
import android.opengl.*;
import android.os.Build;
import android.os.Bundle;
import java.io.FileNotFoundException;
import java.io.FileOutputStream;
import java.io.IOException;
import java.lang.RuntimeException;
import java.nio.ByteBuffer;
import java.util.List;
import java.util.ArrayList;
import java.util.concurrent.Executor;

import android.graphics.ImageFormat;
import android.hardware.camera2.*;
import android.os.Handler;
import android.os.HandlerThread;
import android.util.Range;
import android.util.SparseIntArray;
import android.view.Surface;

import androidx.annotation.NonNull;
import androidx.core.app.ActivityCompat;

// from permission_library
import com.google.vr.sdk.samples.permission.PermissionHelper;

/*
	Custom camera player that renders video to a frame buffer. This
	variation is for API 14 and above.
*/
public class CameraPlayer14
{
	private enum CameraStates {
		PREINIT, INIT, PREPARED, START, PAUSE, STOP, RELEASED
	}

	private boolean CameraSupported = false;
	private boolean mVideoEnabled = true;

    private HandlerThread mBackgroundThread;
    private Handler mBackgroundHandler;

	private CameraManager mCameraManager = null;
	private CameraDevice mCamera = null;
	private CaptureRequest.Builder mPreviewRequestBuilder = null;
	private boolean mAutoStartPreview = false;
	private CameraCaptureSession mCameraCaptureSession = null;
	private final int mCameraFormat = ImageFormat.YUV_420_888; //ImageFormat.JPEG;

	private int DesiredCameraWidth = 0;
	private int DesiredCameraHeight = 0;
	private int DesiredCameraFPS = 0;
	private String DesiredCameraId;

	private CameraStates CameraState = CameraStates.PREINIT;
	private int CameraWidth = 0;
	private int CameraHeight = 0;
	private int CameraFPS = 0;
	private int CameraFPSMin = 0;
	private int CameraFPSMax = 0;
	private int CameraOrientation = 0;
	private int CameraRotationOffset = 0;

	private String CameraId;
	private String CameraFrontId;
	private String CameraBackId;

	private boolean SwizzlePixels = true;
	private boolean VulkanRenderer = false;
	private boolean Looping = false;
	private volatile boolean WaitOnBitmapRender = false;
	private volatile boolean Prepared = false;
	private volatile boolean Completed = false;

	private BitmapRenderer mBitmapRenderer = null;
	private OESTextureRenderer mOESTextureRenderer = null;

	public class FrameUpdateInfo {
		public BitmapRenderer.FrameData BRFrameData = null;
		public java.nio.Buffer Buffer;
		public int CurrentPosition;
		public boolean FrameReady;
		public boolean RegionChanged;
		public float ScaleRotation00;
		public float ScaleRotation01;
		public float ScaleRotation10;
		public float ScaleRotation11;
		public float UOffset;
		public float VOffset;
	}

	public class AudioTrackInfo {
		public int Index;
		public String MimeType;
		public String DisplayName;
		public String Language;
		public int Channels;
		public int SampleRate;
	}

	public class CaptionTrackInfo {
		public int Index;
		public String MimeType;
		public String DisplayName;
		public String Language;
	}

	public class VideoTrackInfo {
		public int Index;
		public String MimeType;
		public String DisplayName;
		public String Language;
		public int BitRate;
		public int Width;
		public int Height;
		public float FrameRate;
		public float FrameRateLow;
		public float FrameRateHigh;
	}

	private ArrayList<AudioTrackInfo> audioTracks = new ArrayList<AudioTrackInfo>();
	private ArrayList<VideoTrackInfo> videoTracks = new ArrayList<VideoTrackInfo>();

	// ======================================================================================

	public CameraPlayer14(boolean swizzlePixels, boolean vulkanRenderer)
	{
		CameraState = CameraStates.INIT;
		SwizzlePixels = swizzlePixels;
		VulkanRenderer = vulkanRenderer;
		WaitOnBitmapRender = false;

		/*
		setOnErrorListener(new MediaPlayer.OnErrorListener() {
			@Override
			public boolean onError(MediaPlayer mp, int what, int extra) {
				GameActivity.Log.debug("MediaPlayer14: onError returned what=" + what + ", extra=" + extra);
				return true;
			}
		});
		*/

		// check if camera supported
		if (GameActivity.Get().getPackageManager().hasSystemFeature(PackageManager.FEATURE_CAMERA_ANY))
		{
			CameraSupported = true;
			GameActivity.Log.debug("Camera supported");
		}
		else
		{
			CameraSupported = false;
			GameActivity.Log.debug("Camera not supported");
			return;
		}

		// make sure we have permission to use camera (user should request this first!)
		if (!PermissionHelper.checkPermission("android.permission.CAMERA"))
		{
			// don't have permission so disable
			CameraSupported = false;
			GameActivity.Log.debug("Camera permission not granted");
			return;
		}		

		// discover available cameras and remember front and back IDs if available
		mCameraManager = (CameraManager) GameActivity.Get().getSystemService(Context.CAMERA_SERVICE);
		try {
			String[] ids = mCameraManager.getCameraIdList();
			if (ids.length <= 0)
			{
				CameraSupported = false;
				GameActivity.Log.debug("CameraPlayer14: No camera is in the device.");
				return;
			}

			CameraFrontId = null;
			CameraBackId = null;
			for (String id : ids)
			{
				CameraCharacteristics characteristics = mCameraManager.getCameraCharacteristics(id);
				int facing = characteristics.get(CameraCharacteristics.LENS_FACING);
				switch (facing)
				{
					case CameraCharacteristics.LENS_FACING_FRONT:
						CameraFrontId = id;
						break;
					case CameraCharacteristics.LENS_FACING_BACK:
						CameraBackId = id;
						break;
				}
				if (CameraFrontId != null && CameraBackId != null)
				{
					break;
				}
			}
			
			if (CameraFrontId == null && CameraBackId == null)
			{
				CameraSupported = false;
				GameActivity.Log.debug("CameraPlayer14: No camera was found suitable for use.");
				return;
			}
		}
		catch (CameraAccessException e)
		{
			CameraSupported = false;
			GameActivity.Log.error("CameraPlayer14: furtherInit exception - " + e.getLocalizedMessage());
			return;
		}

		startBackgroundThread();

		GameActivity.Log.debug("CameraPlayer14: frontId=" + CameraFrontId + ", backId=" + CameraBackId);
	}

	private void startBackgroundThread()
	{
		/* this is to allow the camera opperations to run on a separate thread and avoid blocking the UI*/
		if (mBackgroundThread == null)
		{
			mBackgroundThread = new HandlerThread("Camera2Handler");
			mBackgroundThread.start();
			mBackgroundHandler = new Handler(mBackgroundThread.getLooper());
		}
	}

	private void stopBackgroundThread()
	{
		if (mBackgroundThread == null)
		{
			mBackgroundHandler = null;
			return;
		}
		
		mBackgroundThread.quitSafely();
		try
		{
			mBackgroundThread.join();
		}
		catch (InterruptedException e)
		{
			e.printStackTrace();
		}

		mBackgroundThread = null;
		mBackgroundHandler = null;
	}
	
	public synchronized int getVideoWidth()
	{
//		GameActivity.Log.debug("CameraPlayer14: getVideoWidth: " + CameraWidth);
		return (CameraState == CameraStates.PREPARED ||
				CameraState == CameraStates.START ||
				CameraState == CameraStates.PAUSE ||
				CameraState == CameraStates.STOP) ? CameraWidth : 0;
	}

	public synchronized int getVideoHeight()
	{
//		GameActivity.Log.debug("CameraPlayer14: getVideoHeight: " + CameraHeight);
		return (CameraState == CameraStates.PREPARED ||
				CameraState == CameraStates.START ||
				CameraState == CameraStates.PAUSE ||
				CameraState == CameraStates.STOP) ? CameraHeight : 0;
	}

	public int getFrameRate()
	{
//		GameActivity.Log.debug("CameraPlayer14: getFrameRate: " + CameraFPS);
		return CameraFPS;
	}

	public synchronized void prepare()
	{
		if (null == mCamera)
		{
			return;
		}

//		GameActivity.Log.debug("CameraPlayer14: prepare: " + CameraState);
//		CameraState = CameraStates.PREPARED;
		Prepared = true;
	}

	public synchronized void prepareAsync()
	{
		if (null == mCamera)
		{
			return;
		}

//		GameActivity.Log.debug("CameraPlayer14: prepareAsync: " + CameraState);
//		CameraState = CameraStates.PREPARED;
		Prepared = true;
	}
	
	public synchronized boolean isPlaying()
	{
		return CameraState == CameraStates.START;
	}

	public boolean isPrepared()
	{
		boolean result;
		synchronized(this)
		{
			result = Prepared;
		}
		return result;
	}

	public boolean didComplete()
	{
		boolean result;
		synchronized(this)
		{
			result = Completed;
			Completed = false;
		}
		return result;
	}

	public boolean isLooping()
	{
		return Looping;
	}

	public int getCurrentPosition()
	{
		return 0;
	}

	public int getDuration()
	{
		if (null == mCamera)
		{
			return 0;
		}

		return 1000;
	}

	public void selectTrack(int track)
	{
	}

	public boolean setDataSourceURL(
		String Url)
		throws IOException,
			java.lang.InterruptedException,
			java.util.concurrent.ExecutionException
	{
		if (!CameraSupported)
		{
			GameActivity.Log.error("CameraPlayer14: setDataSourceURL - CameraSupported is false.");
			return false;
		}

//		GameActivity.Log.debug("setDataSource: Url = " + Url);
		
		// defaults
		String cameraId = "";
		int width = 320;
		int height = 480;
		int fps = 30;

		// Url should be in form: "vidcap://camera_id?width=xx?height=xx?fps=xx"
		int index = Url.indexOf("vidcap://");
		if (index >= 0)
		{
			Url = Url.substring(index + 9);
		}
		String[] UrlParts = Url.split("\\?");
		if (UrlParts.length > 0)
		{
//			GameActivity.Log.debug("setDataSource: camera=" + UrlParts[0]);
			if (UrlParts[0].equals("front") && CameraFrontId != null)
			{
				cameraId = CameraFrontId;
//				GameActivity.Log.debug("setDataSource: front=" + iFrontId);
			}
			else if ((UrlParts[0].equals("back") || UrlParts[0].equals("rear")) && CameraBackId != null)
			{
				cameraId = CameraBackId;
//				GameActivity.Log.debug("setDataSource: back=" + iBackId);
			}
			else
			{
				try
				{
					int expectingCameraIndex = Integer.parseInt(UrlParts[0]);
					String[] ids = mCameraManager.getCameraIdList();
					if (expectingCameraIndex >= 0 && expectingCameraIndex < ids.length)
					{
						cameraId = ids[expectingCameraIndex];
					}
					else if (CameraBackId != null)
					{
						cameraId = CameraBackId;
					}
				}
				catch (NumberFormatException | CameraAccessException e)
				{
					throw new RuntimeException(e);
				}
//				GameActivity.Log.debug("setDataSource: back=" + iBackId);
			}

			if (cameraId.length() <= 0 && CameraBackId == null)
			{
				GameActivity.Log.error("CameraPlayer14: setDataSourceURL - Couldn't locate the camera specified in the URL.");
				return false;
			}
			
			for (int UrlPartIndex=1; UrlPartIndex < UrlParts.length; UrlPartIndex++)
			{
				String[] KeyValue = UrlParts[UrlPartIndex].split("=", 2);
				if (KeyValue.length == 2)
				{
					int valueInt;
					try
					{
						valueInt = Integer.parseInt(KeyValue[1]);
					}
					catch (NumberFormatException e)
					{
						continue;
					}
					if (KeyValue[0].equals("width"))
					{
						width = valueInt;
					}
					else if (KeyValue[0].equals("height"))
					{
						height = valueInt;
					}
					else if (KeyValue[0].equals("fps"))
					{
						fps = valueInt;
					}
				}
			}
		}

		DesiredCameraId = cameraId;
		DesiredCameraWidth = width;
		DesiredCameraHeight = height;
		DesiredCameraFPS = fps;
		
		return openCamera();
	}

	public boolean openCamera()
	{
		reset();

		try
		{
			if (!CameraSupported)
			{
				GameActivity.Log.error("CameraPlayer14: openCamera - Camera is not supported.");
				return false;
			}

			if (ActivityCompat.checkSelfPermission(GameActivity.Get(), Manifest.permission.CAMERA) != PackageManager.PERMISSION_GRANTED)
			{
				GameActivity.Log.error("CameraPlayer14: openCamera - Camera permission is not granted.");
				return false;
			}

			GameActivity.Log.debug("CameraPlayer14: openCamera - id=" + DesiredCameraId);

			CameraDevice.StateCallback CameraDeviceStateCallback = new CameraDevice.StateCallback()
				{
					@Override
					public void onOpened(@NonNull CameraDevice camera)
					{
						mCamera = camera;
						configureCamera(DesiredCameraId, DesiredCameraFPS, DesiredCameraWidth, DesiredCameraHeight);
						prepare();
					}

					@Override
					public void onDisconnected(@NonNull CameraDevice camera)
					{
						GameActivity.Log.error("CameraPlayer14: CameraDeviceStateCallback - Camera device disconnected.");
					}

					@Override
					public void onError(@NonNull CameraDevice camera, int error)
					{
						GameActivity.Log.error("CameraPlayer14: CameraDeviceStateCallback - Camera device is in error state - code: " + error);
					}
				};

			mCameraManager.openCamera(DesiredCameraId, CameraDeviceStateCallback, mBackgroundHandler);
		}
		catch (CameraAccessException e)
		{
			GameActivity.Log.error("CameraPlayer14: openCamera - Camera access exception - " + e.getLocalizedMessage());
			return false;
		}

		return true;
	}

	private boolean configureCamera(String cameraId, int fpsDesired, int widthDesired, int heightDesired)
	{
		Range<Integer>[] previewFPSRanges;
		android.util.Size[] previewSizes;

		try
		{
			mPreviewRequestBuilder = mCamera.createCaptureRequest(CameraDevice.TEMPLATE_PREVIEW);

			CameraCharacteristics cameraChar = mCameraManager.getCameraCharacteristics(cameraId);

			StreamConfigurationMap configs = cameraChar.get(CameraCharacteristics.SCALER_STREAM_CONFIGURATION_MAP);
			previewSizes = configs.getOutputSizes(mCameraFormat);

			previewFPSRanges = cameraChar.get(CameraCharacteristics.CONTROL_AE_AVAILABLE_TARGET_FPS_RANGES);

			// fixup for orientation depends on camera facing
			CameraRotationOffset = 0;
			CameraOrientation = cameraChar.get(CameraCharacteristics.SENSOR_ORIENTATION);
			int facing = cameraChar.get(CameraCharacteristics.LENS_FACING);
			if (facing == CameraCharacteristics.LENS_FACING_FRONT)
			{
				CameraRotationOffset = (CameraOrientation + 90) % 360;
			}
			else if (facing == CameraCharacteristics.LENS_FACING_BACK)
			{
				CameraRotationOffset = (450 - CameraOrientation) % 360;
			}
		}
		catch (CameraAccessException e)
		{
			GameActivity.Log.error("CameraPlayer14: configureCamera - camera access exception: " + e.getLocalizedMessage());
			return false;
		}

		if (mPreviewRequestBuilder == null || previewFPSRanges == null || previewSizes == null)
		{
			GameActivity.Log.error("CameraPlayer14: configureCamera - unable to retrieve camera characteristics : " + cameraId);
			return false;
		}

		GameActivity.Log.debug("CameraPlayer14: configureCamera - id=" + cameraId + ", width=" + widthDesired + ", height=" + heightDesired + ", fps=" + fpsDesired);

		// find min and max fps allowed and matched frame rate range
		CameraFPS = fpsDesired;
		CameraFPSMin = fpsDesired * 1000;
		CameraFPSMax = CameraFPSMin;
		float floatMinFps = 10000;
		float floatMaxFps = -10000;
		boolean found = false;

		// first look for an exact match
		for (Range<Integer> range : previewFPSRanges)
		{
			//GameActivity.Log.debug("setDataSource: FPS Range: " + range.getLower() + " - " + range.getUpper());
			float previewMin = range.getLower().floatValue() / 1000.0f;
			float previewMax = range.getUpper().floatValue() / 1000.0f;
			floatMinFps = previewMin < floatMinFps ? previewMin : floatMinFps;
			floatMaxFps = previewMax > floatMaxFps ? previewMax : floatMaxFps;
			if (fpsDesired == range.getLower() && fpsDesired == range.getUpper())
			{
				found = true;
				CameraFPSMin = range.getLower();
				CameraFPSMax = range.getUpper();
			}
		}
		if (!found)
		{
			// look for range containing desired frame rate
			for (Range<Integer> range : previewFPSRanges)
			{
				if (!found || range.contains(fpsDesired))
				{
					found = true;
					CameraFPSMin = range.getLower();
					CameraFPSMax = range.getUpper();
				}
			}
		}

		// find best matched preview size and add track info
		audioTracks.clear();
		videoTracks.clear();

		if (previewSizes.length > 0)
		{
			int bestWidth = -1;
			int bestHeight = -1;
			int bestScore = 0;
			for (int index = 0; index < previewSizes.length; ++index)
			{
				android.util.Size camSize = previewSizes[index];
				//GameActivity.Log.debug("CameraPlayer14: configureCamera - available previewSize " + index + ": " + camSize.getWidth() + " x " + camSize.getHeight());

				VideoTrackInfo videoTrack = new VideoTrackInfo();
				videoTrack.Index = index;
				videoTrack.MimeType = "video/unknown";
				videoTrack.DisplayName = "Video Track " + index + " (Stream " + index + ")";
				videoTrack.Language = "und";
				videoTrack.BitRate = 0;
				videoTrack.Width = camSize.getWidth();
				videoTrack.Height = camSize.getHeight();
				videoTrack.FrameRate = fpsDesired;
				videoTrack.FrameRateLow = floatMinFps;
				videoTrack.FrameRateHigh = floatMaxFps;
				videoTracks.add(videoTrack);

				int diffWidth = camSize.getWidth() - widthDesired;
				int diffHeight = camSize.getHeight() - heightDesired;
				int score = diffWidth * diffWidth + diffHeight * diffHeight;
				if (bestWidth == -1 || score < bestScore)
				{
					bestWidth = camSize.getWidth();
					bestHeight = camSize.getHeight();
					bestScore = score;
				}
			}
			CameraWidth = bestWidth;
			CameraHeight = bestHeight;
			
			GameActivity.Log.debug("CameraPlayer14: configureCamera - The preview size best matched: " + CameraWidth + " x " + CameraHeight);
		}

		// set camera parameters
		mPreviewRequestBuilder.set(CaptureRequest.CONTROL_AE_TARGET_FPS_RANGE, new Range<>(CameraFPSMin, CameraFPSMax));

		// store our picks
		CameraId = cameraId;

		GameActivity.Log.debug("CameraPlayer14: Camera configured - id=" + cameraId + ", width=" + CameraWidth + ", height=" + CameraHeight + ", fps=" + CameraFPS +
				", min=" + CameraFPSMin + " max=" + CameraFPSMax + " orientation=" + CameraOrientation);

		synchronized (this)
		{
			CameraState = CameraStates.PREPARED;
		}
		return true;
	}

	private void CreatePreviewSession(Surface previewSurface)
	{
		mPreviewRequestBuilder.addTarget(previewSurface);

		List<Surface> listSurfaces = new ArrayList<Surface>();
		listSurfaces.add(previewSurface);
		try
		{
			GameActivity.Log.debug("CameraPlayer14: creating capture session...");

			mCamera.createCaptureSession(listSurfaces, new CameraCaptureSession.StateCallback()
			{
				@Override
				public void onConfigured(@NonNull CameraCaptureSession session)
				{
					assert (session.getDevice() == mCamera);

					synchronized (this)
					{
						mCameraCaptureSession = session;
					}

					GameActivity.Log.debug("CameraPlayer14: capture session created!");

					if (mAutoStartPreview)
					{
						mAutoStartPreview = false;
						startPreview();
					}
				}

				@Override
				public void onConfigureFailed(@NonNull CameraCaptureSession session)
				{
					GameActivity.Log.warn("CameraPlayer14: capture session creating failed!");
				}
			}, mBackgroundHandler);
		}
		catch (CameraAccessException e)
		{
			GameActivity.Log.error("CameraPlayer14: CreatePreviewSession - " + e.getLocalizedMessage());
		}
	}

	private void startPreview()
	{
		if (mCameraCaptureSession == null)
		{
			GameActivity.Log.warn("CameraPlayer14: startPreview - Null CaptureSession");
			return;
		}
		try
		{
			GameActivity.Log.debug("CameraPlayer14: Preview starting...");
			mCameraCaptureSession.setRepeatingRequest(mPreviewRequestBuilder.build(), null, mBackgroundHandler);
		}
		catch (CameraAccessException e)
		{
			GameActivity.Log.error("CameraPlayer14: startPreview - " + e.getLocalizedMessage());
		}
	}

	private void stopPreview()
	{
		if (mCameraCaptureSession == null)
		{
			GameActivity.Log.warn("CameraPlayer14: stopPreview - Null CaptureSession");
			return;
		}
		try
		{
			mCameraCaptureSession.stopRepeating();
			GameActivity.Log.debug("CameraPlayer14: Preview stopped");
		}
		catch (CameraAccessException e)
		{
			GameActivity.Log.error("CameraPlayer14: stopPreview - " + e.getLocalizedMessage());
		}
	}

	public String getDataSourceURL()
	{
		String URL = "vidcap://" + ((CameraId == CameraFrontId) ? "front" : "rear") + "?width=" + CameraWidth + "?height=" + CameraHeight + "?fps=" + CameraFPS;
		GameActivity.Log.debug("CameraPlayer14: getDataSource - " + URL);
		return URL;
	}

	public void setVideoEnabled(boolean enabled)
	{
		if (null == mCamera)
		{
			return;
		}

		GameActivity.Log.debug("CameraPlayer14: setVideoEnabled: " + enabled);
		WaitOnBitmapRender = true;

		mVideoEnabled = enabled;
		if (mVideoEnabled)
		{
			if (null != mOESTextureRenderer && null != mOESTextureRenderer.getSurface())
			{
				CreatePreviewSession(mOESTextureRenderer.getSurface());
			}

			if (null != mBitmapRenderer && null != mBitmapRenderer.getSurface())
			{
				CreatePreviewSession(mBitmapRenderer.getSurface());
			}
		}
		else
		{
			stopPreview();
		}

		WaitOnBitmapRender = false;
	}

	public void setAudioEnabled(boolean enabled)
	{
	}

	public boolean didResolutionChange()
	{
		if (null != mOESTextureRenderer)
		{
			return mOESTextureRenderer.resolutionChanged();
		}
		if (null != mBitmapRenderer)
		{
			return mBitmapRenderer.resolutionChanged();
		}
		return false;
	}

	public int getExternalTextureId()
	{
		if (null != mOESTextureRenderer)
		{
			return mOESTextureRenderer.getExternalTextureId();
		}
		if (null != mBitmapRenderer && (mBitmapRenderer instanceof BitmapRendererLegacy))
		{
			return ((BitmapRendererLegacy)mBitmapRenderer).getExternalTextureId();
		}
		return -1;
	}

	public synchronized void start()
	{
//		GameActivity.Log.debug("CameraPlayer14: start: " + CameraState);
		if (CameraState == CameraStates.PREPARED)
		{
			CameraState = CameraStates.PAUSE;
		}

		if (CameraState == CameraStates.PAUSE)
		{
			startPreview();
			CameraState = CameraStates.START;
		}
	}

	public synchronized void pause()
	{
//		GameActivity.Log.debug("CameraPlayer14: pause: " + CameraState);
		if (CameraState == CameraStates.START)
		{
			stopPreview();
		}
		CameraState = CameraStates.PAUSE;
	}

	public synchronized void stop()
	{
//		GameActivity.Log.debug("CameraPlayer14: stop: " + CameraState);
		if (CameraState == CameraStates.START)
		{
			stopPreview();
		}
		CameraState = CameraStates.STOP;
	}

	public void seekTo(int position)
	{
		synchronized (this)
		{
			Completed = false;
		}
	}

	public void setLooping(boolean looping)
	{
		// don't set on player
		Looping = looping;
	}

	public synchronized void release()
	{
		GameActivity.Log.debug("CameraPlayer14: release");
		if (null != mCamera)
		{
			stopPreview();
			mCamera.close();
			mCamera = null;
			mCameraCaptureSession = null;
		}

		Prepared = false;
		
		audioTracks.clear();
		videoTracks.clear();

		CameraState = CameraStates.RELEASED;
	}

	public synchronized void reset()
	{
		GameActivity.Log.debug("CameraPlayer14: reset");

		release();
		
		Completed = false;
		Looping = false;

		if (null != mOESTextureRenderer)
		{
			while (WaitOnBitmapRender) ;
			releaseOESTextureRenderer();
		}
		if (null != mBitmapRenderer)
		{
			while (WaitOnBitmapRender) ;
			releaseBitmapRenderer();
		}

		CameraState = CameraStates.PREINIT;
	}

	// ======================================================================================

	private boolean usingBitmapRendererImageReader = false;
	private static final int ImageReaderQueueLength = 16;
	private BitmapRenderer.UpdateParam updateParam = new BitmapRenderer.UpdateParam();

	public void setUsingBitmapRendererImageReader(boolean enabled)
	{
		usingBitmapRendererImageReader = enabled;
	}

	private boolean CreateBitmapRenderer()
	{
		releaseBitmapRenderer();

		GameActivity.Log.debug("CameraPlayer - CREATE BitmapRenderer, VIDEO SIZE: " + getVideoWidth() + " x " + getVideoHeight() + ", BRIR: " + usingBitmapRendererImageReader + ", Vulkan: " + VulkanRenderer);
		
		if (usingBitmapRendererImageReader)
		{
			mBitmapRenderer = new BitmapRendererImageReader(SwizzlePixels, VulkanRenderer,
					getVideoWidth(), getVideoHeight(), ImageReaderQueueLength);
		}
		else
		{
			mBitmapRenderer = new BitmapRendererLegacy(SwizzlePixels, VulkanRenderer, false);
		}

		if (!mBitmapRenderer.isValid())
		{
			mBitmapRenderer = null;
			return false;
		}
		
		// set this here as the size may have been set before the GL resources were created.
		mBitmapRenderer.setSize(getVideoWidth(),getVideoHeight());

		mAutoStartPreview = true;

		setVideoEnabled(true);
		setAudioEnabled(true);
		return true;
	}

	void releaseBitmapRenderer()
	{
		if (null != mBitmapRenderer)
		{
			mBitmapRenderer.release();
			mBitmapRenderer = null;
		}
	}

	public void initBitmapRenderer()
	{
		// if not already allocated.
		// Create bitmap renderer's gl resources in the renderer thread.
		if (null == mBitmapRenderer)
		{
			if (!CreateBitmapRenderer())
			{
				GameActivity.Log.warn("CameraPlayer14: initBitmapRenderer - failed to alloc mBitmapRenderer");
				reset();
			}
		}
	}

	private float[] mTransformMatrix = new float[16];

	private void UpdateUVTransformation(FrameUpdateInfo updateInfo)
	{
		if (usingBitmapRendererImageReader)
		{
			return;
		}

		android.graphics.SurfaceTexture surfaceTexture = mBitmapRenderer.getSurfaceTexture();
		surfaceTexture.getTransformMatrix(mTransformMatrix);

		int degrees = (GameActivity.Get().getCurrentDeviceRotationDegree() + CameraRotationOffset) % 360;
		if (degrees == 0) 
		{
			updateInfo.ScaleRotation00 = mTransformMatrix[1];
			updateInfo.ScaleRotation01 = mTransformMatrix[5];
			updateInfo.ScaleRotation10 = -mTransformMatrix[0];
			updateInfo.ScaleRotation11 = -mTransformMatrix[4];
			updateInfo.UOffset = mTransformMatrix[13];
			updateInfo.VOffset = 1.0f - mTransformMatrix[12];
			if (CameraId == CameraBackId)
			{
				updateInfo.ScaleRotation00 = -updateInfo.ScaleRotation00;
				updateInfo.ScaleRotation01 = -updateInfo.ScaleRotation01;
				updateInfo.UOffset = 1.0f - updateInfo.UOffset;
			}
		}
		else if (degrees == 90) 
		{
			updateInfo.ScaleRotation00 = mTransformMatrix[0];
			updateInfo.ScaleRotation01 = mTransformMatrix[4];
			updateInfo.ScaleRotation10 = mTransformMatrix[1];
			updateInfo.ScaleRotation11 = mTransformMatrix[5];
			updateInfo.UOffset = mTransformMatrix[12];
			updateInfo.VOffset = mTransformMatrix[13];
			if (CameraId == CameraFrontId)
			{
				updateInfo.ScaleRotation00 = -updateInfo.ScaleRotation00;
				updateInfo.ScaleRotation01 = -updateInfo.ScaleRotation01;
				updateInfo.UOffset = 1.0f - updateInfo.UOffset;
			}
			updateInfo.ScaleRotation10 = -updateInfo.ScaleRotation10;
			updateInfo.ScaleRotation11 = -updateInfo.ScaleRotation11;
			updateInfo.VOffset = 1.0f - updateInfo.VOffset;
		}
		else if (degrees == 180) 
		{
			updateInfo.ScaleRotation00 = mTransformMatrix[1];
			updateInfo.ScaleRotation01 = mTransformMatrix[5];
			updateInfo.ScaleRotation10 = mTransformMatrix[0];
			updateInfo.ScaleRotation11 = mTransformMatrix[4];
			updateInfo.UOffset = mTransformMatrix[13];
			updateInfo.VOffset = mTransformMatrix[12];
			if (CameraId == CameraFrontId)
			{
				updateInfo.ScaleRotation00 = -updateInfo.ScaleRotation00;
				updateInfo.ScaleRotation01 = -updateInfo.ScaleRotation01;
				updateInfo.UOffset = 1.0f - updateInfo.UOffset;
			}
		}
		else 
		{
			updateInfo.ScaleRotation00 = mTransformMatrix[0];
			updateInfo.ScaleRotation01 = mTransformMatrix[4];
			updateInfo.ScaleRotation10 = mTransformMatrix[1];
			updateInfo.ScaleRotation11 = mTransformMatrix[5];
			updateInfo.UOffset = mTransformMatrix[12];
			updateInfo.VOffset = mTransformMatrix[13];
			if (CameraId == CameraBackId)
			{
				updateInfo.ScaleRotation00 = -updateInfo.ScaleRotation00;
				updateInfo.ScaleRotation01 = -updateInfo.ScaleRotation01;
				updateInfo.UOffset = 1.0f - updateInfo.UOffset;
			}
		}
	}

	public FrameUpdateInfo getVideoLastFrameData()
	{
		if (usingBitmapRendererImageReader)
		{
			return null;
		}

		if (null == mCamera)
		{
			return null;
		}

		initBitmapRenderer();
		if (null != mBitmapRenderer)
		{
			WaitOnBitmapRender = true;
			updateParam.destTexture = 0;
			BitmapRenderer.FrameData data = mBitmapRenderer.updateFrameData(updateParam);
			WaitOnBitmapRender = false;
			if (data != null)
			{
				FrameUpdateInfo frameUpdateInfo = new FrameUpdateInfo();
				frameUpdateInfo.Buffer = data.binaryBuffer;
				frameUpdateInfo.BRFrameData = null;
				frameUpdateInfo.FrameReady = true;
				frameUpdateInfo.RegionChanged = false;

				UpdateUVTransformation(frameUpdateInfo);
				
				return frameUpdateInfo;
			}
		}
		return null;
	}

	public FrameUpdateInfo getVideoLastFrame(int destTexture)
	{
		if (usingBitmapRendererImageReader)
		{
			return null;
		}

		if (null == mCamera)
		{
			return null;
		}

//		GameActivity.Log.debug("getVideoLastFrame: " + destTexture);
		initBitmapRenderer();
		if (null != mBitmapRenderer)
		{
			WaitOnBitmapRender = true;
			updateParam.destTexture = destTexture;
			BitmapRenderer.FrameData data = mBitmapRenderer.updateFrameData(updateParam);
			WaitOnBitmapRender = false;
			if (data != null)
			{
				FrameUpdateInfo frameUpdateInfo = new FrameUpdateInfo();
				frameUpdateInfo.BRFrameData = null;
				frameUpdateInfo.Buffer = null;
				frameUpdateInfo.FrameReady = true;
				frameUpdateInfo.RegionChanged = false;
				
				UpdateUVTransformation(frameUpdateInfo);

				return frameUpdateInfo;
			}
		}
		return null;
	}

	public FrameUpdateInfo getVideoLastMediaFrameData()
	{
		if (usingBitmapRendererImageReader)
		{
			initBitmapRenderer();
			if (mBitmapRenderer != null)
			{
				WaitOnBitmapRender = true;
				BitmapRenderer.FrameData data = mBitmapRenderer.updateFrameData(updateParam);
				WaitOnBitmapRender = false;
				if (data != null)
				{
					FrameUpdateInfo frameUpdateInfo = new FrameUpdateInfo();
					frameUpdateInfo.BRFrameData = data;
					frameUpdateInfo.Buffer = null;
					frameUpdateInfo.FrameReady = true;
					frameUpdateInfo.RegionChanged = false;
					return frameUpdateInfo;
				}
			}
		}
		return null;
	}

	// ======================================================================================

	private boolean CreateOESTextureRenderer(int OESTextureId)
	{
		releaseOESTextureRenderer();

		GameActivity.Log.debug("CameraPlayer - CREATE OESTextureRenderer, TextureId: " + OESTextureId);
		
		mOESTextureRenderer = new OESTextureRenderer(OESTextureId);
		if (!mOESTextureRenderer.isValid())
		{
			mOESTextureRenderer = null;
			return false;
		}
		
		// set this here as the size may have been set before the GL resources were created.
		mOESTextureRenderer.setSize(getVideoWidth(),getVideoHeight());

		mAutoStartPreview = true;

		setVideoEnabled(true);
		setAudioEnabled(true);
		return true;
	}

	void releaseOESTextureRenderer()
	{
		if (null != mOESTextureRenderer)
		{
			mOESTextureRenderer.release();
			mOESTextureRenderer = null;
		}
	}
	
	public FrameUpdateInfo updateVideoFrame(int externalTextureId)
	{
		if (null == mOESTextureRenderer)
		{
			if (!CreateOESTextureRenderer(externalTextureId))
			{
				GameActivity.Log.warn("CameraPlayer14: updateVideoFrame - failed to alloc mOESTextureRenderer");
				reset();
				return null;
			}
		}

		WaitOnBitmapRender = true;
		FrameUpdateInfo result = mOESTextureRenderer.updateVideoFrame();
		WaitOnBitmapRender = false;
		return result;
	}

	/*
		This handles events for our OES texture
	*/
	class OESTextureRenderer
		implements android.graphics.SurfaceTexture.OnFrameAvailableListener
	{
		private android.graphics.SurfaceTexture mSurfaceTexture = null;
		private int mTextureWidth = -1;
		private int mTextureHeight = -1;
		private android.view.Surface mSurface = null;
		private boolean mFrameAvailable = false;
		private int mTextureID = -1;
		private float[] mTransformMatrix = new float[16];
		private boolean mTextureSizeChanged = true;
		private int GL_TEXTURE_EXTERNAL_OES = 0x8D65;

		private float mScaleRotation00 = 1.0f;
		private float mScaleRotation01 = 0.0f;
		private float mScaleRotation10 = 0.0f;
		private float mScaleRotation11 = 1.0f;
		private float mUOffset = 0.0f;
		private float mVOffset = 0.0f;

		public OESTextureRenderer(int OESTextureId)
		{
			mTextureID = OESTextureId;

			mSurfaceTexture = new android.graphics.SurfaceTexture(mTextureID);
			mSurfaceTexture.setOnFrameAvailableListener(this);
			mSurface = new android.view.Surface(mSurfaceTexture);
		}

		public void release()
		{
			if (null != mSurface)
			{
				mSurface.release();
				mSurface = null;
			}
			if (null != mSurfaceTexture)
			{
				mSurfaceTexture.release();
				mSurfaceTexture = null;
			}
		}

		public boolean isValid()
		{
			return mSurfaceTexture != null;
		}

		public void onFrameAvailable(android.graphics.SurfaceTexture st)
		{
			synchronized(this)
			{
				mFrameAvailable = true;
			}
		}

		public android.graphics.SurfaceTexture getSurfaceTexture()
		{
			return mSurfaceTexture;
		}

		public android.view.Surface getSurface()
		{
			return mSurface;
		}

		public int getExternalTextureId()
		{
			return mTextureID;
		}

		// NOTE: Synchronized with updateFrameData to prevent frame
		// updates while the surface may need to get reallocated.
		public void setSize(int width, int height)
		{
			synchronized(this)
			{
				if (width != mTextureWidth ||
					height != mTextureHeight)
				{
					mTextureWidth = width;
					mTextureHeight = height;
					mTextureSizeChanged = true;
				}
			}
		}

		public boolean resolutionChanged()
		{
			boolean changed;
			synchronized(this)
			{
				changed = mTextureSizeChanged;
				mTextureSizeChanged = false;
			}
			return changed;
		}

		public FrameUpdateInfo updateVideoFrame()
		{
			synchronized(this)
			{
				return getFrameUpdateInfo();
			}
		}

		private FrameUpdateInfo getFrameUpdateInfo()
		{
			FrameUpdateInfo frameUpdateInfo = new FrameUpdateInfo();

			frameUpdateInfo.Buffer = null;
			frameUpdateInfo.CurrentPosition = getCurrentPosition();
			frameUpdateInfo.FrameReady = false;
			frameUpdateInfo.RegionChanged = false;

			frameUpdateInfo.ScaleRotation00 = mScaleRotation00;
			frameUpdateInfo.ScaleRotation01 = mScaleRotation01;
			frameUpdateInfo.ScaleRotation10 = mScaleRotation10;
			frameUpdateInfo.ScaleRotation11 = mScaleRotation11;

			frameUpdateInfo.UOffset = mUOffset;
			frameUpdateInfo.VOffset = mVOffset;

			if (!mFrameAvailable)
			{
				// We only return fresh data when we generate it. At other
				// time we return nothing to indicate that there was nothing
				// new to return. The media player deals with this by keeping
				// the last frame around and using that for rendering.
				return frameUpdateInfo;
			}
			mFrameAvailable = false;
			if (null == mSurfaceTexture)
			{
				// Can't update if there's no surface to update into.
				return frameUpdateInfo;
			}

			frameUpdateInfo.FrameReady = true;

			// Get the latest video texture frame.
			mSurfaceTexture.updateTexImage();
			mSurfaceTexture.getTransformMatrix(mTransformMatrix);

			/*
			GameActivity.Log.debug(mTransformMatrix[0] + ", " + mTransformMatrix[1] + ", " + mTransformMatrix[2] + ", " + mTransformMatrix[3]);
			GameActivity.Log.debug(mTransformMatrix[4] + ", " + mTransformMatrix[5] + ", " + mTransformMatrix[6] + ", " + mTransformMatrix[7]);
			GameActivity.Log.debug(mTransformMatrix[8] + ", " + mTransformMatrix[9] + ", " + mTransformMatrix[10] + ", " + mTransformMatrix[11]);
			GameActivity.Log.debug(mTransformMatrix[12] + ", " + mTransformMatrix[13] + ", " + mTransformMatrix[14] + ", " + mTransformMatrix[15]);
			*/

			int degrees = (GameActivity.Get().getCurrentDeviceRotationDegree() + CameraRotationOffset) % 360;
			if (degrees == 0) {
				mScaleRotation00 = mTransformMatrix[1];
				mScaleRotation01 = mTransformMatrix[5];
				mScaleRotation10 = -mTransformMatrix[0];
				mScaleRotation11 = -mTransformMatrix[4];
				mUOffset = mTransformMatrix[13];
				mVOffset = 1.0f - mTransformMatrix[12];
				if (CameraId == CameraBackId)
				{
					mScaleRotation00 = -mScaleRotation00;
					mScaleRotation01 = -mScaleRotation01;
					mUOffset = 1.0f - mUOffset;
				}
			}
			else if (degrees == 90) {
				mScaleRotation00 = mTransformMatrix[0];
				mScaleRotation01 = mTransformMatrix[4];
				mScaleRotation10 = mTransformMatrix[1];
				mScaleRotation11 = mTransformMatrix[5];
				mUOffset = mTransformMatrix[12];
				mVOffset = mTransformMatrix[13];
				if (CameraId == CameraFrontId)
				{
					mScaleRotation00 = -mScaleRotation00;
					mScaleRotation01 = -mScaleRotation01;
					mUOffset = 1.0f - mUOffset;
				}
				mScaleRotation10 = -mScaleRotation10;
				mScaleRotation11 = -mScaleRotation11;
				mVOffset = 1.0f - mVOffset;
			}
			else if (degrees == 180) {
				mScaleRotation00 = mTransformMatrix[1];
				mScaleRotation01 = mTransformMatrix[5];
				mScaleRotation10 = mTransformMatrix[0];
				mScaleRotation11 = mTransformMatrix[4];
				mUOffset = mTransformMatrix[13];
				mVOffset = mTransformMatrix[12];
				if (CameraId == CameraFrontId)
				{
					mScaleRotation00 = -mScaleRotation00;
					mScaleRotation01 = -mScaleRotation01;
					mUOffset = 1.0f - mUOffset;
				}
			}
			else {
				mScaleRotation00 = mTransformMatrix[0];
				mScaleRotation01 = mTransformMatrix[4];
				mScaleRotation10 = mTransformMatrix[1];
				mScaleRotation11 = mTransformMatrix[5];
				mUOffset = mTransformMatrix[12];
				mVOffset = mTransformMatrix[13];
				if (CameraId == CameraBackId)
				{
					mScaleRotation00 = -mScaleRotation00;
					mScaleRotation01 = -mScaleRotation01;
					mUOffset = 1.0f - mUOffset;
				}
			}

			if (frameUpdateInfo.ScaleRotation00 != mScaleRotation00 ||
				frameUpdateInfo.ScaleRotation01 != mScaleRotation01 ||
				frameUpdateInfo.ScaleRotation10 != mScaleRotation10 ||
				frameUpdateInfo.ScaleRotation11 != mScaleRotation11 ||
				frameUpdateInfo.UOffset != mUOffset ||
				frameUpdateInfo.VOffset != mVOffset)
			{
				frameUpdateInfo.RegionChanged = true;

				frameUpdateInfo.ScaleRotation00 = mScaleRotation00;
				frameUpdateInfo.ScaleRotation01 = mScaleRotation01;
				frameUpdateInfo.ScaleRotation10 = mScaleRotation10;
				frameUpdateInfo.ScaleRotation11 = mScaleRotation11;

				frameUpdateInfo.UOffset = mUOffset;
				frameUpdateInfo.VOffset = mVOffset;
			}

			
			// updateTexImage binds an external texture to active texture unit
			// make sure to unbind it to prevent state leak
			GLES20.glBindTexture(GL_TEXTURE_EXTERNAL_OES, 0);
			
			return frameUpdateInfo;
		}
	};

	// ======================================================================================

	public AudioTrackInfo[] GetAudioTracks()
	{
		AudioTrackInfo[] AudioTracks = new AudioTrackInfo[0];

		return AudioTracks;
	}

	public CaptionTrackInfo[] GetCaptionTracks()
	{
		CaptionTrackInfo[] CaptionTracks = new CaptionTrackInfo[0];

		return CaptionTracks;
	}

	public VideoTrackInfo[] GetVideoTracks()
	{
		int Width = getVideoWidth();
		int Height = getVideoHeight();

		int CountTracks = videoTracks.size();
		if (CountTracks > 0)
		{
			VideoTrackInfo[] VideoTracks = new VideoTrackInfo[CountTracks];
			int TrackIndex = 0;
			for (VideoTrackInfo track : videoTracks)
			{
				VideoTracks[TrackIndex] = new VideoTrackInfo();
				VideoTracks[TrackIndex].Index = track.Index;
				VideoTracks[TrackIndex].MimeType = track.MimeType;
				VideoTracks[TrackIndex].DisplayName = track.DisplayName;
				VideoTracks[TrackIndex].Language = track.Language;
				VideoTracks[TrackIndex].BitRate = track.BitRate;
				VideoTracks[TrackIndex].Width = track.Width;
				VideoTracks[TrackIndex].Height = track.Height;
				VideoTracks[TrackIndex].FrameRate = track.FrameRate;
				VideoTracks[TrackIndex].FrameRateLow = track.FrameRateLow;
				VideoTracks[TrackIndex].FrameRateHigh = track.FrameRateHigh;
				//GameActivity.Log.debug(TrackIndex + "Video: Mime=" + VideoTracks[TrackIndex].MimeType + ", Lang=" + VideoTracks[TrackIndex].Language + ", Width=" + VideoTracks[TrackIndex].Width + ", Height=" + VideoTracks[TrackIndex].Height + ", FrameRate=" + VideoTracks[TrackIndex].FrameRate);
				TrackIndex++;
			}

			return VideoTracks;
		}

		if (Width > 0 && Height > 0)
		{
			VideoTrackInfo[] VideoTracks = new VideoTrackInfo[1];

			VideoTracks[0] = new VideoTrackInfo();
			VideoTracks[0].Index = 0;
			VideoTracks[0].DisplayName = "Video Track 0 (Stream 0)";
			VideoTracks[0].Language = "und";
			VideoTracks[0].BitRate = 0;
			VideoTracks[0].MimeType = "video/unknown";
			VideoTracks[0].Width = Width;
			VideoTracks[0].Height = Height;
			VideoTracks[0].FrameRate = CameraFPS;

			return VideoTracks;
		}

		VideoTrackInfo[] VideoTracks = new VideoTrackInfo[0];
		GameActivity.Log.error("CameraPlayer14: Empty video tracks.");

		return VideoTracks;
	}

	// ======================================================================================

	private static final SparseIntArray ORIENTATIONS = new SparseIntArray();

	static {
		ORIENTATIONS.append(Surface.ROTATION_0, 0);
		ORIENTATIONS.append(Surface.ROTATION_90, 90);
		ORIENTATIONS.append(Surface.ROTATION_180, 180);
		ORIENTATIONS.append(Surface.ROTATION_270, 270);
	}
	
	private int sensorToDeviceRotation(String cameraId, int deviceOrientation)
	{
		try
		{
			CameraCharacteristics cameraChar = mCameraManager.getCameraCharacteristics(cameraId);
			int sensorOrientation = cameraChar.get(CameraCharacteristics.SENSOR_ORIENTATION);
			// Get device orientation in degrees
			deviceOrientation = ORIENTATIONS.get(deviceOrientation);
			// Reverse device orientation for front-facing cameras
			if (cameraChar.get(CameraCharacteristics.LENS_FACING) == CameraCharacteristics.LENS_FACING_FRONT)
			{
				deviceOrientation = -deviceOrientation;
			}
			// Calculate desired JPEG orientation relative to camera orientation to make
			// the image upright relative to the device orientation
			return (sensorOrientation + deviceOrientation + 360) % 360;
		}
		catch (CameraAccessException e)
		{
			throw new RuntimeException(e);
		}
	}

	public boolean takePicture(String Filename, int width, int height)
	{
		synchronized (this)
		{
			if (null == mCamera || CameraState != CameraStates.START)
			{
				GameActivity.Log.error("CameraPlayer14: takePicture - Camera not ready!");
				return false;
			}
		}

		if (width <= 0 || height <= 0)
		{
			width = CameraWidth;
			height = CameraHeight;
			GameActivity.Log.warn("CameraPlayer14: takePicture - invalid width or height, using current camera size: " + width + "x" + height);
		}

		try
		{
			// find best matched picture size
			int bestWidth = -1;
			int bestHeight = -1;
			CameraCharacteristics cameraChar = mCameraManager.getCameraCharacteristics(CameraId);
			StreamConfigurationMap configs = cameraChar.get(CameraCharacteristics.SCALER_STREAM_CONFIGURATION_MAP);
			android.util.Size[] pictureSizes = configs.getOutputSizes(ImageFormat.JPEG);
			if (pictureSizes.length > 0)
			{
				int bestScore = 0;
				for (int index = 0; index < pictureSizes.length; ++index)
				{
					android.util.Size camSize = pictureSizes[index];
					int diffWidth = camSize.getWidth() - width;
					int diffHeight = camSize.getHeight() - height;
					int score = diffWidth * diffWidth + diffHeight * diffHeight;
					if (bestWidth == -1 || score < bestScore)
					{
						bestWidth = camSize.getWidth();
						bestHeight = camSize.getHeight();
						bestScore = score;
					}
				}
				GameActivity.Log.debug("CameraPlayer14: takePicture - final size: " + bestWidth + " x " + bestHeight);
			}
			else
			{
				bestWidth = width;
				bestHeight = height;
			}

			stopPreview();

			final ImageReader.OnImageAvailableListener readerListener = reader -> {
				Image image = reader.acquireNextImage();
				ByteBuffer buffer = image.getPlanes()[0].getBuffer();
				byte[] bytes = new byte[buffer.capacity()];
				buffer.get(bytes);
				try
				{
					FileOutputStream outStream = new FileOutputStream(Filename);
					outStream.write(bytes);
					outStream.close();
				}
				catch (IOException e)
				{
					throw new RuntimeException(e);
				}
				finally
				{
					image.close();
				}
			};

			ImageReader reader = ImageReader.newInstance(bestWidth, bestHeight, ImageFormat.JPEG, 1);
			reader.setOnImageAvailableListener(readerListener, mBackgroundHandler);

			CaptureRequest.Builder captureRequestBuilder = mCamera.createCaptureRequest(CameraDevice.TEMPLATE_STILL_CAPTURE);
			captureRequestBuilder.addTarget(reader.getSurface());
			captureRequestBuilder.set(CaptureRequest.CONTROL_MODE, CaptureRequest.CONTROL_MODE_AUTO);
			int rotation = GameActivity.Get().getDisplay().getRotation();
			int totalRotation = sensorToDeviceRotation(CameraId, rotation);
			captureRequestBuilder.set(CaptureRequest.JPEG_ORIENTATION, totalRotation);			

			final CameraCaptureSession.CaptureCallback captureListener = new CameraCaptureSession.CaptureCallback()
			{
				@Override
				public void onCaptureCompleted(CameraCaptureSession session, CaptureRequest request, TotalCaptureResult result)
				{
					super.onCaptureCompleted(session, request, result);
					synchronized (this)
					{
						CameraState = CameraStates.PAUSE;
						mCameraCaptureSession = null;
						setVideoEnabled(true);
					}
				}
			};

			List<Surface> outputSurfaces = new ArrayList<Surface>(1);
			outputSurfaces.add(reader.getSurface());
			mCamera.createCaptureSession(outputSurfaces, new CameraCaptureSession.StateCallback()
			{
				@Override
				public void onConfigured(CameraCaptureSession session)
				{
					try
					{
						session.capture(captureRequestBuilder.build(), captureListener, mBackgroundHandler);
					}
					catch (CameraAccessException e)
					{
						e.printStackTrace();
					}
				}

				@Override
				public void onConfigureFailed(CameraCaptureSession session) {}
			}, mBackgroundHandler);
		}
		catch (CameraAccessException e)
		{
			throw new RuntimeException(e);
		}

		return true;
	}
}
