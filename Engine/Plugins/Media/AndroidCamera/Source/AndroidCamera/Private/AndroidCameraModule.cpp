// Copyright Epic Games, Inc. All Rights Reserved.

#include "AndroidCameraPrivate.h"

#include "Modules/ModuleManager.h"

#include "IAndroidCameraModule.h"
#include "Player/AndroidCameraPlayer.h"

#include "IMediaCaptureSupport.h"
#include "IMediaModule.h"

#include "Android/AndroidJNI.h"
#include "Android/AndroidApplication.h"

DEFINE_LOG_CATEGORY(LogAndroidCamera);


/**
 * Implements the AndroidCamera module.
 */
class FAndroidCameraModule
	: public IMediaCaptureSupport
	, public IAndroidCameraModule
{
public:

	//~ IMediaCaptureDevices interface

	virtual void EnumerateAudioCaptureDevices(TArray<FMediaCaptureDeviceInfo>& OutDeviceInfos) override
	{
	}

	virtual void EnumerateVideoCaptureDevices(TArray<FMediaCaptureDeviceInfo>& OutDeviceInfos) override
	{
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FAndroidCameraPlayer::EnumerateVideoCaptureDevices"));

		if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
		{
			static jmethodID CountMethod = FJavaWrapper::FindMethod(Env, FJavaWrapper::GameActivityClassID, "AndroidThunkJava_AndroidCamera_CountCameras", "()I", false);
			int CountCameras = FJavaWrapper::CallIntMethod(Env, FJavaWrapper::GameActivityThis, CountMethod);

			static jmethodID QueryMethod = FJavaWrapper::FindMethod(Env, FJavaWrapper::GameActivityClassID, "AndroidThunkJava_AndroidCamera_GetCameraUrl", "(I)Ljava/lang/String;", false);
			for (int CameraIndex = 0; CameraIndex < CountCameras; ++CameraIndex)
			{
				const auto CameraUrl = FJavaHelper::FStringFromLocalRef(Env, (jstring)FJavaWrapper::CallObjectMethod(Env, FJavaWrapper::GameActivityThis, QueryMethod, CameraIndex));
				if (CameraUrl.Len())
				{
					FMediaCaptureDeviceInfo DeviceInfo;
					DeviceInfo.Url = CameraUrl;
					if (DeviceInfo.Url.Contains("front"))
					{
						DeviceInfo.Type = EMediaCaptureDeviceType::WebcamFront;
						DeviceInfo.Info = TEXT("Android front camera");
					}
					else if (DeviceInfo.Url.Contains("rear"))
					{
						DeviceInfo.Type = EMediaCaptureDeviceType::WebcamRear;
						DeviceInfo.Info = TEXT("Android back camera");
					}
					else
					{
						DeviceInfo.Type = EMediaCaptureDeviceType::Webcam;
						DeviceInfo.Info = TEXT("Android camera");
					}
//					DeviceInfo.Type = EMediaCaptureDeviceType::Video;
					DeviceInfo.DisplayName = FText::FromString(DeviceInfo.Info);

					OutDeviceInfos.Add(MoveTemp(DeviceInfo));
				}
			}
		}
	}

	//~ IAndroidCameraModule interface

	virtual TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> CreatePlayer(IMediaEventSink& EventSink) override
	{
		if (!IsSupported())
		{
			return nullptr;
		}

		return MakeShared<FAndroidCameraPlayer, ESPMode::ThreadSafe>(EventSink);
	}

public:

	//~ IModuleInterface interface

	virtual void StartupModule() override
	{
		// register capture device support
		auto MediaModule = FModuleManager::LoadModulePtr<IMediaModule>("Media");

		if (MediaModule != nullptr)
		{
			MediaModule->RegisterCaptureSupport(*this);
		}

		Initialized = true;
	}
	
	virtual void ShutdownModule() override
	{
		if (!Initialized)
		{
			return;
		}

		// unregister capture support
		auto MediaModule = FModuleManager::GetModulePtr<IMediaModule>("Media");

		if (MediaModule != nullptr)
		{
			MediaModule->UnregisterCaptureSupport(*this);
		}

		Initialized = false;
	}

private:

	/** Whether the module has been initialized. */
	bool Initialized;

protected:

	/**
	 * Check whether media is supported on the running device.
	 *
	 * @return true if media is supported, false otherwise.
	 */
	bool IsSupported()
	{
		return (FAndroidMisc::GetAndroidBuildVersion() >= 14);
	}
};


IMPLEMENT_MODULE(FAndroidCameraModule, AndroidCamera)
