// Copyright Epic Games, Inc. All Rights Reserved.

#include "AndroidJavaCameraPlayer.h"
#include "Android/AndroidApplication.h"

#if UE_BUILD_SHIPPING
// always clear any exceptions in SHipping
#define CHECK_JNI_RESULT(Id) if (Id == 0) { JEnv->ExceptionClear(); }
#else
#define CHECK_JNI_RESULT(Id) \
if (Id == 0) \
{ \
	if (!bIsOptional) { JEnv->ExceptionDescribe(); checkf(Id != 0, TEXT("Failed to find " #Id)); } \
	JEnv->ExceptionClear(); \
}
#endif

static jfieldID FindField(JNIEnv* JEnv, jclass Class, const ANSICHAR* FieldName, const ANSICHAR* FieldType, bool bIsOptional)
{
	jfieldID Field = Class == NULL ? NULL : JEnv->GetFieldID(Class, FieldName, FieldType);
	CHECK_JNI_RESULT(Field);
	return Field;
}

bool AndroidEnableCameraPlayerBRIR = true;
static FAutoConsoleVariableRef CVarAndroidEnableCameraPlayerBRIR(
	TEXT("Android.EnableCameraPlayerBRIR"),
	AndroidEnableCameraPlayerBRIR,
	TEXT("Determines how the Android Camera Player handles video frame data. True for using ImageReader/HardwareBuffer. False for copying data through the CPU memory. (Default: true)"),
	ECVF_Default);

FJavaAndroidCameraPlayer::FJavaAndroidCameraPlayer(bool swizzlePixels, bool vulkanRenderer)
	: FJavaClassObject(GetClassName(), "(ZZ)V", swizzlePixels, vulkanRenderer)
	, GetDurationMethod(GetClassMethod("getDuration", "()I"))
	, ResetMethod(GetClassMethod("reset", "()V"))
	, GetCurrentPositionMethod(GetClassMethod("getCurrentPosition", "()I"))
	, DidCompleteMethod(GetClassMethod("didComplete", "()Z"))
	, IsLoopingMethod(GetClassMethod("isLooping", "()Z"))
	, IsPlayingMethod(GetClassMethod("isPlaying", "()Z"))
	, IsPreparedMethod(GetClassMethod("isPrepared", "()Z"))
	, SetDataSourceURLMethod(GetClassMethod("setDataSourceURL", "(Ljava/lang/String;)Z"))
	, GetDataSourceURLMethod(GetClassMethod("getDataSourceURL", "()Ljava/lang/String;"))
	, SeekToMethod(GetClassMethod("seekTo", "(I)V"))
	, SetLoopingMethod(GetClassMethod("setLooping", "(Z)V"))
	, OpenCameraMethod(GetClassMethod("openCamera", "()Z"))
	, ReleaseMethod(GetClassMethod("release", "()V"))
	, GetVideoHeightMethod(GetClassMethod("getVideoHeight", "()I"))
	, GetVideoWidthMethod(GetClassMethod("getVideoWidth", "()I"))
	, GetFrameRateMethod(GetClassMethod("getFrameRate", "()I"))
	, SetVideoEnabledMethod(GetClassMethod("setVideoEnabled", "(Z)V"))
	, SetAudioEnabledMethod(GetClassMethod("setAudioEnabled", "(Z)V"))
	, GetVideoLastFrameDataMethod(GetClassMethod("getVideoLastFrameData", "()Lcom/epicgames/unreal/CameraPlayer14$FrameUpdateInfo;"))
	, StartMethod(GetClassMethod("start", "()V"))
	, PauseMethod(GetClassMethod("pause", "()V"))
	, StopMethod(GetClassMethod("stop", "()V"))
	, GetVideoLastFrameMethod(GetClassMethod("getVideoLastFrame", "(I)Lcom/epicgames/unreal/CameraPlayer14$FrameUpdateInfo;"))
	, GetAudioTracksMethod(GetClassMethod("GetAudioTracks", "()[Lcom/epicgames/unreal/CameraPlayer14$AudioTrackInfo;"))
	, GetCaptionTracksMethod(GetClassMethod("GetCaptionTracks", "()[Lcom/epicgames/unreal/CameraPlayer14$CaptionTrackInfo;"))
	, GetVideoTracksMethod(GetClassMethod("GetVideoTracks", "()[Lcom/epicgames/unreal/CameraPlayer14$VideoTrackInfo;"))
	, DidResolutionChangeMethod(GetClassMethod("didResolutionChange", "()Z"))
	, GetExternalTextureIdMethod(GetClassMethod("getExternalTextureId", "()I"))
	, UpdateVideoFrameMethod(GetClassMethod("updateVideoFrame", "(I)Lcom/epicgames/unreal/CameraPlayer14$FrameUpdateInfo;"))
	, TakePictureMethod(GetClassMethod("takePicture", "(Ljava/lang/String;II)Z"))
	, SetUsingBitmapRendererImageReaderMethod(GetClassMethod("setUsingBitmapRendererImageReader", "(Z)V"))
	, GetVideoLastMediaFrameDataMethod(GetClassMethod("getVideoLastMediaFrameData", "()Lcom/epicgames/unreal/CameraPlayer14$FrameUpdateInfo;"))
{
	VideoTexture = nullptr;
	bVideoTextureValid = false;

	ScaleRotation.X = 1.0f;
	ScaleRotation.Y = 0.0f;
	ScaleRotation.Z = 0.0f;
	ScaleRotation.W = 1.0f;
	Offset.X = 0.0f;
	Offset.Y = 0.0f;
	Offset.Z = 0.0f;
	Offset.W = 0.0f;

	bTrackInfoSupported = FAndroidMisc::GetAndroidBuildVersion() >= 16;

	PlayerState = FPlayerState::Inactive;

	if (bTrackInfoSupported)
	{
		SelectTrackMethod = GetClassMethod("selectTrack", "(I)V");
	}

	BRSelection = BitmapRendererSelection::Unknown;
	UsingVulkanRenderer = vulkanRenderer && FAndroidMisc::ShouldUseVulkan();

	JNIEnv* JEnv = FAndroidApplication::GetJavaEnv();

	// get field IDs for FrameUpdateInfo class members
	FrameUpdateInfoClass = FAndroidApplication::FindJavaClassGlobalRef("com/epicgames/unreal/CameraPlayer14$FrameUpdateInfo");
	FrameUpdateInfo_Buffer = FindField(JEnv, FrameUpdateInfoClass, "Buffer", "Ljava/nio/Buffer;", false);
	FrameUpdateInfo_CurrentPosition = FindField(JEnv, FrameUpdateInfoClass, "CurrentPosition", "I", false);
	FrameUpdateInfo_FrameReady = FindField(JEnv, FrameUpdateInfoClass, "FrameReady", "Z", false);
	FrameUpdateInfo_RegionChanged = FindField(JEnv, FrameUpdateInfoClass, "RegionChanged", "Z", false);
	FrameUpdateInfo_ScaleRotation00 = FindField(JEnv, FrameUpdateInfoClass, "ScaleRotation00", "F", false);
	FrameUpdateInfo_ScaleRotation01 = FindField(JEnv, FrameUpdateInfoClass, "ScaleRotation01", "F", false);
	FrameUpdateInfo_ScaleRotation10 = FindField(JEnv, FrameUpdateInfoClass, "ScaleRotation10", "F", false);
	FrameUpdateInfo_ScaleRotation11 = FindField(JEnv, FrameUpdateInfoClass, "ScaleRotation11", "F", false);
	FrameUpdateInfo_UOffset = FindField(JEnv, FrameUpdateInfoClass, "UOffset", "F", false);
	FrameUpdateInfo_VOffset = FindField(JEnv, FrameUpdateInfoClass, "VOffset", "F", false);
	FrameUpdateInfo_BRFrameData = FindField(JEnv, FrameUpdateInfoClass, "BRFrameData", "Lcom/epicgames/unreal/BitmapRenderer$FrameData;", false);

	// get field IDs for AudioTrackInfo class members
	AudioTrackInfoClass = FAndroidApplication::FindJavaClassGlobalRef("com/epicgames/unreal/CameraPlayer14$AudioTrackInfo");
	AudioTrackInfo_Index = FindField(JEnv, AudioTrackInfoClass, "Index", "I", false);
	AudioTrackInfo_MimeType = FindField(JEnv, AudioTrackInfoClass, "MimeType", "Ljava/lang/String;", false);
	AudioTrackInfo_DisplayName = FindField(JEnv, AudioTrackInfoClass, "DisplayName", "Ljava/lang/String;", false);
	AudioTrackInfo_Language = FindField(JEnv, AudioTrackInfoClass, "Language", "Ljava/lang/String;", false);
	AudioTrackInfo_Channels = FindField(JEnv, AudioTrackInfoClass, "Channels", "I", false);
	AudioTrackInfo_SampleRate = FindField(JEnv, AudioTrackInfoClass, "SampleRate", "I", false);

	// get field IDs for CaptionTrackInfo class members
	CaptionTrackInfoClass = FAndroidApplication::FindJavaClassGlobalRef("com/epicgames/unreal/CameraPlayer14$CaptionTrackInfo");
	CaptionTrackInfo_Index = FindField(JEnv, CaptionTrackInfoClass, "Index", "I", false);
	CaptionTrackInfo_MimeType = FindField(JEnv, CaptionTrackInfoClass, "MimeType", "Ljava/lang/String;", false);
	CaptionTrackInfo_DisplayName = FindField(JEnv, CaptionTrackInfoClass, "DisplayName", "Ljava/lang/String;", false);
	CaptionTrackInfo_Language = FindField(JEnv, CaptionTrackInfoClass, "Language", "Ljava/lang/String;", false);

	// get field IDs for VideoTrackInfo class members
	VideoTrackInfoClass = FAndroidApplication::FindJavaClassGlobalRef("com/epicgames/unreal/CameraPlayer14$VideoTrackInfo");
	VideoTrackInfo_Index = FindField(JEnv, VideoTrackInfoClass, "Index", "I", false);
	VideoTrackInfo_MimeType = FindField(JEnv, VideoTrackInfoClass, "MimeType", "Ljava/lang/String;", false);
	VideoTrackInfo_DisplayName = FindField(JEnv, VideoTrackInfoClass, "DisplayName", "Ljava/lang/String;", false);
	VideoTrackInfo_Language = FindField(JEnv, VideoTrackInfoClass, "Language", "Ljava/lang/String;", false);
	VideoTrackInfo_BitRate = FindField(JEnv, VideoTrackInfoClass, "BitRate", "I", false);
	VideoTrackInfo_Width = FindField(JEnv, VideoTrackInfoClass, "Width", "I", false);
	VideoTrackInfo_Height = FindField(JEnv, VideoTrackInfoClass, "Height", "I", false);
	VideoTrackInfo_FrameRate = FindField(JEnv, VideoTrackInfoClass, "FrameRate", "F", false);
	VideoTrackInfo_FrameRateLow = FindField(JEnv, VideoTrackInfoClass, "FrameRateLow", "F", false);
	VideoTrackInfo_FrameRateHigh = FindField(JEnv, VideoTrackInfoClass, "FrameRateHigh", "F", false);
}

FJavaAndroidCameraPlayer::~FJavaAndroidCameraPlayer()
{
	if (auto Env = FAndroidApplication::GetJavaEnv())
	{
		Env->DeleteGlobalRef(FrameUpdateInfoClass);
		Env->DeleteGlobalRef(AudioTrackInfoClass);
		Env->DeleteGlobalRef(CaptionTrackInfoClass);
		Env->DeleteGlobalRef(VideoTrackInfoClass);
	}
}

int32 FJavaAndroidCameraPlayer::GetDuration()
{
	return CallMethod<int32>(GetDurationMethod);
}

bool FJavaAndroidCameraPlayer::IsActive()
{
	return PlayerState == FPlayerState::Active;
}

void FJavaAndroidCameraPlayer::Reset()
{
	PlayerState = FPlayerState::Inactive;

	ScaleRotation.X = 1.0f;
	ScaleRotation.Y = 0.0f;
	ScaleRotation.Z = 0.0f;
	ScaleRotation.W = 1.0f;
	Offset.X = 0.0f;
	Offset.Y = 0.0f;

	CallMethod<void>(ResetMethod);
}

void FJavaAndroidCameraPlayer::Stop()
{
	CallMethod<void>(StopMethod);
}

int32 FJavaAndroidCameraPlayer::GetCurrentPosition()
{
	int32 position = CallMethod<int32>(GetCurrentPositionMethod);
	return position;
}

bool FJavaAndroidCameraPlayer::IsLooping()
{
	return CallMethod<bool>(IsLoopingMethod);
}

bool FJavaAndroidCameraPlayer::IsPlaying()
{
	return CallMethod<bool>(IsPlayingMethod);
}

bool FJavaAndroidCameraPlayer::IsPrepared()
{
	return CallMethod<bool>(IsPreparedMethod);
}

bool FJavaAndroidCameraPlayer::DidComplete()
{
	return CallMethod<bool>(DidCompleteMethod);
}

bool FJavaAndroidCameraPlayer::SetDataSource(const FString & Url)
{
	ScaleRotation.X = 1.0f;
	ScaleRotation.Y = 0.0f;
	ScaleRotation.Z = 0.0f;
	ScaleRotation.W = 1.0f;
	Offset.X = 0.0f;
	Offset.Y = 0.0f;
	
	bool Result = CallMethod<bool>(SetDataSourceURLMethod, *GetJString(Url));

	if (Result)
	{
		PlayerState = FPlayerState::Active;
	}
	return Result;
}

FString FJavaAndroidCameraPlayer::GetDataSource()
{
	return CallMethod<FString>(GetDataSourceURLMethod);
}

bool FJavaAndroidCameraPlayer::Prepare()
{
	// This can return an exception in some cases (URL without internet, for example)
	JNIEnv*	JEnv = FAndroidApplication::GetJavaEnv();
	JEnv->CallVoidMethod(Object, PrepareMethod.Method);
	if (JEnv->ExceptionCheck())
	{
		JEnv->ExceptionDescribe();
		JEnv->ExceptionClear();
		return false;
	}
	return true;
}

bool FJavaAndroidCameraPlayer::PrepareAsync()
{
	// This can return an exception in some cases (URL without internet, for example)
	JNIEnv*	JEnv = FAndroidApplication::GetJavaEnv();
	JEnv->CallVoidMethod(Object, PrepareAsyncMethod.Method);
	if (JEnv->ExceptionCheck())
	{
		JEnv->ExceptionDescribe();
		JEnv->ExceptionClear();
		return false;
	}
	return true;
}

void FJavaAndroidCameraPlayer::SeekTo(int32 Milliseconds)
{
	CallMethod<void>(SeekToMethod, Milliseconds);
}

void FJavaAndroidCameraPlayer::SetLooping(bool Looping)
{
	CallMethod<void>(SetLoopingMethod, Looping);
}

bool FJavaAndroidCameraPlayer::RestoreCamera()
{
	return CallMethod<bool>(OpenCameraMethod);
}

void FJavaAndroidCameraPlayer::Release()
{
	CallMethod<void>(ReleaseMethod);
}

int32 FJavaAndroidCameraPlayer::GetVideoHeight()
{
	return CallMethod<int32>(GetVideoHeightMethod);
}

int32 FJavaAndroidCameraPlayer::GetVideoWidth()
{
	return CallMethod<int32>(GetVideoWidthMethod);
}

float FJavaAndroidCameraPlayer::GetFrameRate()
{
	return (float)CallMethod<int32>(GetFrameRateMethod);
}

void FJavaAndroidCameraPlayer::SetVideoEnabled(bool enabled /*= true*/)
{
	CallMethod<void>(SetVideoEnabledMethod, enabled);
}

void FJavaAndroidCameraPlayer::SetAudioEnabled(bool enabled /*= true*/)
{
	CallMethod<void>(SetAudioEnabledMethod, enabled);
}

void FJavaAndroidCameraPlayer::UpdateScaleRotationOffset()
{
	static TMap<EDeviceScreenOrientation, const FVector2f/*X->Sin, Y->Cos*/> OrientationFactorsMap;
	if (OrientationFactorsMap.IsEmpty())
	{
		OrientationFactorsMap.Add(EDeviceScreenOrientation::Portrait, FVector2f(-1.0f, 0.0f));
		OrientationFactorsMap.Add(EDeviceScreenOrientation::PortraitUpsideDown, FVector2f(1.0f, 0.0f));
		OrientationFactorsMap.Add(EDeviceScreenOrientation::LandscapeRight, FVector2f(0.0f, -1.0f));
		OrientationFactorsMap.Add(EDeviceScreenOrientation::LandscapeLeft, FVector2f(0.0f, 1.0f));
	}

	EDeviceScreenOrientation DeviceOrientation = FAndroidMisc::GetDeviceOrientation();
	const FVector2f* SinCos = OrientationFactorsMap.Find(DeviceOrientation);
	if (SinCos != nullptr)
	{
		ScaleRotation.Set(SinCos->Y, -SinCos->X, SinCos->X, SinCos->Y);
		FVector2f TranslationOffset(-0.5f, -0.5f);
		Offset.X = TranslationOffset.Dot(FVector2f(ScaleRotation.X, ScaleRotation.Y)) - TranslationOffset.X;
		Offset.Y = TranslationOffset.Dot(FVector2f(ScaleRotation.Z, ScaleRotation.W)) - TranslationOffset.Y;
		Offset.Z = Offset.W = 0.0f;
	}
	else
	{
		ScaleRotation.Set(1.0f, 0.0f, 0.0f, 1.0f);
		Offset.Set(0.0f, 0.0f, 0.0f, 0.0f);
	}
}

bool FJavaAndroidCameraPlayer::GetVideoLastFrameData(void* & outPixels, int64 & outCount, int32 *CurrentPosition, bool *bRegionChanged)
{
	if (IsUsingBitmapRendererImageReader())
	{
		return false;
	}

	// This can return an exception in some cases
	JNIEnv*	JEnv = FAndroidApplication::GetJavaEnv();
	auto Result = NewScopedJavaObject(JEnv, JEnv->CallObjectMethod(Object, GetVideoLastFrameDataMethod.Method));
	if (JEnv->ExceptionCheck())
	{
		JEnv->ExceptionDescribe();
		JEnv->ExceptionClear();
		*CurrentPosition = -1;
		*bRegionChanged = false;
		return false;
	}

	if (!Result)
	{
		return false;
	}

	auto buffer = NewScopedJavaObject(JEnv, JEnv->GetObjectField(*Result, FrameUpdateInfo_Buffer));
	if (buffer)
	{
		*CurrentPosition = (int32)JEnv->GetIntField(*Result, FrameUpdateInfo_CurrentPosition);
		bool bFrameReady = (bool)JEnv->GetBooleanField(*Result, FrameUpdateInfo_FrameReady);
		*bRegionChanged = (bool)JEnv->GetBooleanField(*Result, FrameUpdateInfo_RegionChanged);

		UpdateScaleRotationOffset();

		outPixels = JEnv->GetDirectBufferAddress(*buffer);
		outCount = JEnv->GetDirectBufferCapacity(*buffer);

		return !(nullptr == outPixels || 0 == outCount);
	}
	
	return false;
}

/** This function has to be called before calling any GetVideoLastXXXXXXX() functions */
void FJavaAndroidCameraPlayer::SelectBitmapRenderer()
{
	if (BRSelection == BitmapRendererSelection::Unknown)
	{
		if (UsingVulkanRenderer)
		{
			if (AndroidEnableCameraPlayerBRIR && FAndroidMisc::IsExternalMemoryAndroidHardwareBufferExtensionLoaded())
			{
				BRSelection = BitmapRendererSelection::ImageReader;
			}
			else
			{
				BRSelection = BitmapRendererSelection::Legacy;
			}
		}
		else
		{
			if (AndroidEnableCameraPlayerBRIR)
			{
				BRSelection = BitmapRendererSelection::ImageReader;
			}
			else
			{
				BRSelection = BitmapRendererSelection::Legacy;
			}
		}

		JNIEnv* JEnv = FAndroidApplication::GetJavaEnv();
		JEnv->CallVoidMethod(Object, SetUsingBitmapRendererImageReaderMethod.Method, (BRSelection == BitmapRendererSelection::ImageReader));
	}
}

bool FJavaAndroidCameraPlayer::IsUsingBitmapRendererImageReader()
{
	SelectBitmapRenderer();
	return (BRSelection == BitmapRendererSelection::ImageReader);
}

jobject FJavaAndroidCameraPlayer::GetVideoLastMediaFrameData(int32& CurrentPosition, bool& bRegionChanged)
{
	bRegionChanged = false;

	if (!IsUsingBitmapRendererImageReader())
	{
		return nullptr;
	}

	JNIEnv* JEnv = FAndroidApplication::GetJavaEnv();
	jobject FrameData = JEnv->CallObjectMethod(Object, GetVideoLastMediaFrameDataMethod.Method);
	auto FrameUpdateInfo = NewScopedJavaObject(JEnv, FrameData);
	if (JEnv->ExceptionCheck())
	{
		JEnv->ExceptionDescribe();
		JEnv->ExceptionClear();
		return nullptr;
	}

	if (FrameUpdateInfo)
	{
		jobject BRFrameData = JEnv->GetObjectField(*FrameUpdateInfo, FrameUpdateInfo_BRFrameData);
		if (BRFrameData)
		{
			CurrentPosition = (int32)JEnv->GetIntField(*FrameUpdateInfo, FrameUpdateInfo_CurrentPosition);
			bool bFrameReady = (bool)JEnv->GetBooleanField(*FrameUpdateInfo, FrameUpdateInfo_FrameReady);
			bRegionChanged = (bool)JEnv->GetBooleanField(*FrameUpdateInfo, FrameUpdateInfo_RegionChanged);

			UpdateScaleRotationOffset();

			jobject GlobalBRFrameData = JEnv->NewGlobalRef(BRFrameData);
			JEnv->DeleteLocalRef(BRFrameData);
			return GlobalBRFrameData;
		}
	}

	return nullptr;
}

void FJavaAndroidCameraPlayer::Start()
{
	CallMethod<void>(StartMethod);
}

void FJavaAndroidCameraPlayer::Pause()
{
	CallMethod<void>(PauseMethod);
}

bool FJavaAndroidCameraPlayer::DidResolutionChange()
{
	return CallMethod<bool>(DidResolutionChangeMethod);
}

int32 FJavaAndroidCameraPlayer::GetExternalTextureId()
{
	return CallMethod<int32>(GetExternalTextureIdMethod);
}

bool FJavaAndroidCameraPlayer::UpdateVideoFrame(int32 ExternalTextureId, int32 *CurrentPosition, bool *bRegionChanged)
{
	// This can return an exception in some cases
	JNIEnv*	JEnv = FAndroidApplication::GetJavaEnv();
	auto Result = NewScopedJavaObject(JEnv, JEnv->CallObjectMethod(Object, UpdateVideoFrameMethod.Method, ExternalTextureId));
	if (JEnv->ExceptionCheck())
	{
		JEnv->ExceptionDescribe();
		JEnv->ExceptionClear();
		*CurrentPosition = -1;
		*bRegionChanged = false;
		return false;
	}

	if (!Result)
	{
		*CurrentPosition = -1;
		*bRegionChanged = false;
		return false;
	}

	*CurrentPosition = (int32)JEnv->GetIntField(*Result, FrameUpdateInfo_CurrentPosition);
	bool bFrameReady = (bool)JEnv->GetBooleanField(*Result, FrameUpdateInfo_FrameReady);
	*bRegionChanged = (bool)JEnv->GetBooleanField(*Result, FrameUpdateInfo_RegionChanged);

	UpdateScaleRotationOffset();

	return bFrameReady;
}

bool FJavaAndroidCameraPlayer::GetVideoLastFrame(int32 destTexture)
{
	if (IsUsingBitmapRendererImageReader())
	{
		return false;
	}

	// This can return an exception in some cases
	JNIEnv*	JEnv = FAndroidApplication::GetJavaEnv();
	auto Result = NewScopedJavaObject(JEnv, JEnv->CallObjectMethod(Object, GetVideoLastFrameMethod.Method, destTexture));
	if (JEnv->ExceptionCheck())
	{
		JEnv->ExceptionDescribe();
		JEnv->ExceptionClear();
		return false;
	}

	if (!Result)
	{
		return false;
	}

	bool bFrameReady = (bool)JEnv->GetBooleanField(*Result, FrameUpdateInfo_FrameReady);

	UpdateScaleRotationOffset();
	
	return bFrameReady;
}

bool FJavaAndroidCameraPlayer::TakePicture(const FString& Filename)
{
	return TakePicture(Filename, 0, 0);
}

bool FJavaAndroidCameraPlayer::TakePicture(const FString& Filename, int32 Width, int32 Height)
{
	return CallMethod<bool>(TakePictureMethod, *GetJString(Filename), Width, Height);
}

FName FJavaAndroidCameraPlayer::GetClassName()
{
	if (FAndroidMisc::GetAndroidBuildVersion() >= 14)
	{
		return FName("com/epicgames/unreal/CameraPlayer14");
	}
	else
	{
		return FName("");
	}
}

bool FJavaAndroidCameraPlayer::SelectTrack(int32 index)
{
	if (!bTrackInfoSupported)
	{
		// Just assume it worked
		return true;
	}

	// This can return an exception in some cases
	JNIEnv*	JEnv = FAndroidApplication::GetJavaEnv();
	JEnv->CallVoidMethod(Object, SelectTrackMethod.Method, index);
	if (JEnv->ExceptionCheck())
	{
		JEnv->ExceptionDescribe();
		JEnv->ExceptionClear();
		return false;
	}
	return true;
}

bool FJavaAndroidCameraPlayer::GetAudioTracks(TArray<FAudioTrack>& AudioTracks)
{
	AudioTracks.Empty();

	jobjectArray TrackArray = CallMethod<jobjectArray>(GetAudioTracksMethod);
	if (nullptr != TrackArray)
	{
		bool bIsOptional = false;
		JNIEnv* JEnv = FAndroidApplication::GetJavaEnv();
		jsize ElementCount = JEnv->GetArrayLength(TrackArray);

		for (int Index = 0; Index < ElementCount; ++Index)
		{
			auto Track = NewScopedJavaObject(JEnv, JEnv->GetObjectArrayElement(TrackArray, Index));

			int32 AudioTrackIndex = AudioTracks.AddDefaulted();
			FAudioTrack& AudioTrack = AudioTracks[AudioTrackIndex];

			AudioTrack.Index = (int32)JEnv->GetIntField(*Track, AudioTrackInfo_Index);

			AudioTrack.MimeType = FJavaHelper::FStringFromLocalRef(JEnv, (jstring)JEnv->GetObjectField(*Track, AudioTrackInfo_MimeType));
			AudioTrack.DisplayName = FJavaHelper::FStringFromLocalRef(JEnv, (jstring)JEnv->GetObjectField(*Track, AudioTrackInfo_DisplayName));
			AudioTrack.Language = FJavaHelper::FStringFromLocalRef(JEnv, (jstring)JEnv->GetObjectField(*Track, AudioTrackInfo_Language));
			
			AudioTrack.Channels = (int32)JEnv->GetIntField(*Track, AudioTrackInfo_Channels);
			AudioTrack.SampleRate = (int32)JEnv->GetIntField(*Track, AudioTrackInfo_SampleRate);
		}
		JEnv->DeleteGlobalRef(TrackArray);

		return true;
	}
	return false;
}

bool FJavaAndroidCameraPlayer::GetCaptionTracks(TArray<FCaptionTrack>& CaptionTracks)
{
	CaptionTracks.Empty();

	jobjectArray TrackArray = CallMethod<jobjectArray>(GetCaptionTracksMethod);
	if (nullptr != TrackArray)
	{
		bool bIsOptional = false;
		JNIEnv* JEnv = FAndroidApplication::GetJavaEnv();
		jsize ElementCount = JEnv->GetArrayLength(TrackArray);

		for (int Index = 0; Index < ElementCount; ++Index)
		{
			auto Track = NewScopedJavaObject(JEnv, JEnv->GetObjectArrayElement(TrackArray, Index));

			int32 CaptionTrackIndex = CaptionTracks.AddDefaulted();
			FCaptionTrack& CaptionTrack = CaptionTracks[CaptionTrackIndex];

			CaptionTrack.Index = (int32)JEnv->GetIntField(*Track, CaptionTrackInfo_Index);

			CaptionTrack.MimeType = FJavaHelper::FStringFromLocalRef(JEnv, (jstring)JEnv->GetObjectField(*Track, CaptionTrackInfo_MimeType));
			CaptionTrack.DisplayName = FJavaHelper::FStringFromLocalRef(JEnv, (jstring)JEnv->GetObjectField(*Track, CaptionTrackInfo_DisplayName));
			CaptionTrack.Language = FJavaHelper::FStringFromLocalRef(JEnv, (jstring)JEnv->GetObjectField(*Track, CaptionTrackInfo_Language));
		}
		JEnv->DeleteGlobalRef(TrackArray);

		return true;
	}
	return false;
}

bool FJavaAndroidCameraPlayer::GetVideoTracks(TArray<FVideoTrack>& VideoTracks)
{
	VideoTracks.Empty();

	jobjectArray TrackArray = CallMethod<jobjectArray>(GetVideoTracksMethod);
	if (nullptr != TrackArray)
	{
		bool bIsOptional = false;
		JNIEnv* JEnv = FAndroidApplication::GetJavaEnv();
		jsize ElementCount = JEnv->GetArrayLength(TrackArray);

		if (ElementCount > 0)
		{
			auto Track = NewScopedJavaObject(JEnv, JEnv->GetObjectArrayElement(TrackArray, 0));

			int32 VideoTrackIndex = VideoTracks.AddDefaulted();
			FVideoTrack& VideoTrack = VideoTracks[VideoTrackIndex];

			VideoTrack.Index = (int32)JEnv->GetIntField(*Track, VideoTrackInfo_Index);

			VideoTrack.MimeType = FJavaHelper::FStringFromLocalRef(JEnv, (jstring)JEnv->GetObjectField(*Track, VideoTrackInfo_MimeType));
			VideoTrack.DisplayName = FJavaHelper::FStringFromLocalRef(JEnv, (jstring)JEnv->GetObjectField(*Track, VideoTrackInfo_DisplayName));
			VideoTrack.Language = FJavaHelper::FStringFromLocalRef(JEnv, (jstring)JEnv->GetObjectField(*Track, VideoTrackInfo_Language));

			VideoTrack.BitRate = (int32)JEnv->GetIntField(*Track, VideoTrackInfo_BitRate);
			VideoTrack.Dimensions = FIntPoint(GetVideoWidth(), GetVideoHeight());
			VideoTrack.FrameRate = GetFrameRate();
			VideoTrack.FrameRates = TRange<float>(JEnv->GetFloatField(*Track, VideoTrackInfo_FrameRateLow), JEnv->GetFloatField(*Track, VideoTrackInfo_FrameRateHigh));
			VideoTrack.Format = 0;

			for (int Index = 0; Index < ElementCount; ++Index)
			{
				auto Format = NewScopedJavaObject(JEnv, JEnv->GetObjectArrayElement(TrackArray, Index));

				int32 VideoFormatIndex = VideoTrack.Formats.AddDefaulted();
				FVideoFormat& VideoFormat = VideoTrack.Formats[VideoFormatIndex];

				VideoFormat.Dimensions = FIntPoint((int32)JEnv->GetIntField(*Format, VideoTrackInfo_Width), (int32)JEnv->GetIntField(*Format, VideoTrackInfo_Height));
				VideoFormat.FrameRate = JEnv->GetFloatField(*Format, VideoTrackInfo_FrameRateHigh);
				VideoFormat.FrameRates = TRange<float>(JEnv->GetFloatField(*Format, VideoTrackInfo_FrameRateLow), JEnv->GetFloatField(*Format, VideoTrackInfo_FrameRateHigh));

				if (VideoTrack.Dimensions == VideoFormat.Dimensions)
				{
					VideoTrack.Format = VideoFormatIndex;
					VideoFormat.FrameRate = VideoTrack.FrameRate;
				}
			}
		}
		JEnv->DeleteGlobalRef(TrackArray);

		return true;
	}
	return false;
}

