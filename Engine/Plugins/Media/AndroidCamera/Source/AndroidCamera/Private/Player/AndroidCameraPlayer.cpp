// Copyright Epic Games, Inc. All Rights Reserved.

#include "AndroidCameraPlayer.h"
#include "AndroidCameraPrivate.h"

#include "Android/AndroidPlatformFile.h"
#include "AndroidCameraSettings.h"
#include "IMediaEventSink.h"
#include "MediaSamples.h"
#include "Misc/CoreDelegates.h"
#include "Misc/Guid.h"
#include "Misc/Paths.h"
#include "RenderingThread.h"
#include "RHI.h"
#include "RHIResources.h"
#include "UObject/Class.h"
#include "UObject/UObjectGlobals.h"
#include "ExternalTexture.h"

#include "AndroidJavaCameraPlayer.h"
#include "AndroidCameraTextureSample.h"

#define ANDROIDCAMERAPLAYER_USE_EXTERNALTEXTURE 1
#define ANDROIDCAMERAPLAYER_USE_NATIVELOGGING 1

#define LOCTEXT_NAMESPACE "FAndroidCameraPlayer"


/* FAndroidCameraPlayer structors
 *****************************************************************************/

FAndroidCameraPlayer::FAndroidCameraPlayer(IMediaEventSink& InEventSink)
	: CurrentState(EMediaState::Closed)
	, CurrentTime(FTimespan::Zero())
	, bLooping(false)
	, EventSink(InEventSink)
#if WITH_ENGINE
	, JavaCameraPlayer(MakeShared<FJavaAndroidCameraPlayer, ESPMode::ThreadSafe>(false, FAndroidMisc::ShouldUseVulkan()))
#else
	, JavaCameraPlayer(MakeShared<FJavaAndroidCameraPlayer, ESPMode::ThreadSafe>(true, FAndroidMisc::ShouldUseVulkan()))
#endif
	, Samples(MakeShared<FMediaSamples, ESPMode::ThreadSafe>())
	, SelectedAudioTrack(INDEX_NONE)
	, SelectedCaptionTrack(INDEX_NONE)
	, SelectedVideoTrack(INDEX_NONE)
	, VideoSamplePool(new FAndroidCameraTextureSamplePool)
	, bOpenWithoutEvents(false)
	, IsInBackground(false)
	, DeferredSetRateThread()
	, IsSetRateThreadRunning(false)
	, StopSetRateThread(false)
{
	check(JavaCameraPlayer.IsValid());
	check(Samples.IsValid());

	CurrentState = (JavaCameraPlayer.IsValid() && Samples.IsValid())? EMediaState::Closed : EMediaState::Error;
}


FAndroidCameraPlayer::~FAndroidCameraPlayer()
{
	Close();

	if (JavaCameraPlayer.IsValid())
	{
		if (GSupportsImageExternal && !FAndroidMisc::ShouldUseVulkan())
		{
			// Unregister the external texture on render thread
			FTextureRHIRef VideoTexture = JavaCameraPlayer->GetVideoTexture();

			JavaCameraPlayer->SetVideoTexture(nullptr);
			JavaCameraPlayer->Reset();
			JavaCameraPlayer->Release();

			struct FReleaseVideoResourcesParams
			{
				FTextureRHIRef VideoTexture;
				FGuid PlayerGuid;
			};

			FReleaseVideoResourcesParams Params = { VideoTexture, PlayerGuid };

			ENQUEUE_RENDER_COMMAND(AndroidCameraPlayerWriteVideoSample)(
				[Params](FRHICommandListImmediate& RHICmdList)
				{
					#if ANDROIDCAMERAPLAYER_USE_NATIVELOGGING
						FPlatformMisc::LowLevelOutputDebugStringf(TEXT("~FAndroidCameraPlayer: Unregister Guid: %s"), *Params.PlayerGuid.ToString());
					#endif

					FExternalTextureRegistry::Get().UnregisterExternalTexture(Params.PlayerGuid);

					// @todo: this causes a crash
//					Params.VideoTexture->Release();
				});
		}
		else
		{
			JavaCameraPlayer->SetVideoTexture(nullptr);
			JavaCameraPlayer->Reset();
			JavaCameraPlayer->Release();
		}
	}

	delete VideoSamplePool;
	VideoSamplePool = nullptr;
}


/* IMediaPlayer interface
 *****************************************************************************/

void FAndroidCameraPlayer::Close()
{
	#if ANDROIDCAMERAPLAYER_USE_NATIVELOGGING
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FAndroidCamera::Close() - %s"), *PlayerGuid.ToString());
	#endif

	if (CurrentState == EMediaState::Closed)
	{
		return;
	}

	CurrentState = EMediaState::Closed;

	// remove delegates if registered
	if (ResumeHandle.IsValid())
	{
		FCoreDelegates::ApplicationHasEnteredForegroundDelegate.Remove(ResumeHandle);
		ResumeHandle.Reset();
	}

	if (PauseHandle.IsValid())
	{
		FCoreDelegates::ApplicationWillEnterBackgroundDelegate.Remove(PauseHandle);
		PauseHandle.Reset();
	}

	CurrentTime = FTimespan::Zero();

	bLooping = false;

	if (JavaCameraPlayer.IsValid())
	{
		JavaCameraPlayer->SetLooping(false);
//		JavaCameraPlayer->SetVideoTextureValid(false);
		JavaCameraPlayer->Stop();
		JavaCameraPlayer->Reset();
	}

	VideoSamplePool->Reset();

	SelectedAudioTrack = INDEX_NONE;
	SelectedCaptionTrack = INDEX_NONE;
	SelectedVideoTrack = INDEX_NONE;

	AudioTracks.Empty();
	CaptionTracks.Empty();
	VideoTracks.Empty();

	Info.Empty();
	MediaUrl = FString();

	// notify listeners
	if (!bOpenWithoutEvents)
	{
		EventSink.ReceiveMediaEvent(EMediaEvent::TracksChanged);
		EventSink.ReceiveMediaEvent(EMediaEvent::MediaClosed);
	}
}


IMediaCache& FAndroidCameraPlayer::GetCache()
{
	return *this;
}


IMediaControls& FAndroidCameraPlayer::GetControls()
{
	return *this;
}


FString FAndroidCameraPlayer::GetInfo() const
{
	return Info;
}


FGuid FAndroidCameraPlayer::GetPlayerPluginGUID() const
{
	static FGuid PlayerPluginGUID(0x94f59418, 0xb93d4f40, 0xa441b4fd, 0x2ac7c313);
	return PlayerPluginGUID;
}


IMediaSamples& FAndroidCameraPlayer::GetSamples()
{
	return *Samples.Get();
}


FString FAndroidCameraPlayer::GetStats() const
{
	return TEXT("AndroidCamera stats information not implemented yet");
}


IMediaTracks& FAndroidCameraPlayer::GetTracks()
{
	return *this;
}


FString FAndroidCameraPlayer::GetUrl() const
{
	return MediaUrl;
}


IMediaView& FAndroidCameraPlayer::GetView()
{
	return *this;
}


bool FAndroidCameraPlayer::Open(const FString& Url, const IMediaOptions* /*Options*/)
{
	#if ANDROIDCAMERAPLAYER_USE_NATIVELOGGING
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FAndroidCamera::Open(%s) - %s"), *Url, *PlayerGuid.ToString());
	#endif

	if (CurrentState == EMediaState::Error)
	{
		return false;
	}

	Close();

	if ((Url.IsEmpty()))
	{
		return false;
	}

	MediaUrl = Url;

	// open the media
	if (Url.StartsWith(TEXT("vidcam://")))
	{
		FString FilePath = Url.RightChop(9);

		if (!JavaCameraPlayer->SetDataSource(FilePath))
		{
			UE_LOGF(LogAndroidCamera, Warning, "Failed to set data source for vidcam %ls", *FilePath);
			return false;
		}
	}
	else
	{
		if (!JavaCameraPlayer->SetDataSource(Url))
		{
			UE_LOGF(LogAndroidCamera, Warning, "Failed to set data source for URL %ls", *Url);
			return false;
		}
	}

	// prepare media
	MediaUrl = JavaCameraPlayer->GetDataSource();

	CurrentState = EMediaState::Preparing;
	return true;
}


bool FAndroidCameraPlayer::Open(const TSharedRef<FArchive, ESPMode::ThreadSafe>& Archive, const FString& OriginalUrl, const IMediaOptions* /*Options*/)
{
	return false; // not supported
}


void FAndroidCameraPlayer::SetGuid(const FGuid& Guid)
{
	PlayerGuid = Guid;

	#if ANDROIDCAMERAPLAYER_USE_NATIVELOGGING
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("IMediaPlayer SetGuid: %s"), *PlayerGuid.ToString());
	#endif
}

// ==============================================================
// Used by TickFetch

static void ConditionalInitializeVideoTexture_RenderThread(FRHICommandListBase& RHICmdList, TWeakPtr<FJavaAndroidCameraPlayer, ESPMode::ThreadSafe> JavaCameraPlayerPtr, FGuid PlayerGuid)
{
	auto PinnedJavaCameraPlayer = JavaCameraPlayerPtr.Pin();

	if (!PinnedJavaCameraPlayer.IsValid())
	{
		return;
	}

	FTextureRHIRef VideoTexture = PinnedJavaCameraPlayer->GetVideoTexture();
	if (VideoTexture == nullptr)
	{
		const FRHITextureCreateDesc Desc =
			FRHITextureCreateDesc::Create2D(TEXT("VideoTexture"), 1, 1, PF_R8G8B8A8)
			.SetFlags(ETextureCreateFlags::External)
			.SetInitialState(ERHIAccess::SRVMask);

		VideoTexture = RHICmdList.CreateTexture(Desc);
		PinnedJavaCameraPlayer->SetVideoTexture(VideoTexture);

		if (VideoTexture == nullptr)
		{
			UE_LOGF(LogAndroidCamera, Warning, "CreateTexture failed!");
			return;
		}

		PinnedJavaCameraPlayer->SetVideoTextureValid(false);

#if ANDROIDCAMERAPLAYER_USE_NATIVELOGGING
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Fetch RT: Created VideoTexture: %s"), *PlayerGuid.ToString());
#endif
	}
}

static void DoUpdateExternalCameraSampleExecute(TWeakPtr<FJavaAndroidCameraPlayer, ESPMode::ThreadSafe> JavaCameraPlayerPtr, FGuid PlayerGuid)
{
	auto PinnedJavaCameraPlayer = JavaCameraPlayerPtr.Pin();

	if (!PinnedJavaCameraPlayer.IsValid())
	{
		return;
	}

	FTextureRHIRef VideoTexture = PinnedJavaCameraPlayer->GetVideoTexture();
	if (VideoTexture == nullptr)
	{
		return;
	}

	int32 TextureId = *reinterpret_cast<int32*>(VideoTexture->GetNativeResource());
	int32 CurrentFramePosition = 0;
	bool bRegionChanged = false;
	if (PinnedJavaCameraPlayer->UpdateVideoFrame(TextureId, &CurrentFramePosition, &bRegionChanged))
	{
#if ANDROIDCAMERAPLAYER_USE_NATIVELOGGING
//		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Fetch RT: %d - %s"), CurrentFramePosition, *PlayerGuid.ToString());
#endif

		// if region changed, need to reregister UV scale/offset
		if (bRegionChanged)
		{
			PinnedJavaCameraPlayer->SetVideoTextureValid(false);
		}
	}
}

struct FRHICommandUpdateExternalCameraSample final : public FRHICommand<FRHICommandUpdateExternalCameraSample>
{
	TWeakPtr<FJavaAndroidCameraPlayer, ESPMode::ThreadSafe> JavaCameraPlayerPtr;
	FGuid PlayerGuid;

	FORCEINLINE_DEBUGGABLE FRHICommandUpdateExternalCameraSample(TWeakPtr<FJavaAndroidCameraPlayer, ESPMode::ThreadSafe> InJavaCameraPlayerPtr, FGuid InPlayerGuid)
		: JavaCameraPlayerPtr(InJavaCameraPlayerPtr)
		, PlayerGuid(InPlayerGuid)
	{
	}
	void Execute(FRHICommandListBase& CmdList)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FRHICommandUpdateExternalCameraSample_Execute);
		DoUpdateExternalCameraSampleExecute(JavaCameraPlayerPtr, PlayerGuid);
	}
};

static void ConditionalInitializeVideoSample_RenderThread(FRHICommandListBase& RHICmdList, TSharedRef<FAndroidCameraTextureSample, ESPMode::ThreadSafe> VideoSample)
{
	if (VideoSample->GetTexture() == nullptr || VideoSample->GetTexture()->GetSizeXY() != VideoSample->GetOutputDim())
	{
		VideoSample->InitializeTexture(RHICmdList, FTimespan::Zero());
	}
}

static void DoUpdateTextureCameraSampleExecute(FRHICommandListBase& RHICmdList, TWeakPtr<FJavaAndroidCameraPlayer, ESPMode::ThreadSafe> JavaCameraPlayerPtr, TWeakPtr<FMediaSamples, ESPMode::ThreadSafe> SamplesPtr,
	TSharedRef<FAndroidCameraTextureSample, ESPMode::ThreadSafe> VideoSample, FTimespan SampleTime)
{
	auto PinnedJavaCameraPlayer = JavaCameraPlayerPtr.Pin();
	auto PinnedSamples = SamplesPtr.Pin();

	if (!PinnedJavaCameraPlayer.IsValid() || !PinnedSamples.IsValid())
	{
		return;
	}

	// write frame into texture
	FRHITexture* Texture = VideoSample->InitializeTexture(RHICmdList, SampleTime);

	if (Texture != nullptr)
	{
		int32 Resource = *reinterpret_cast<int32*>(Texture->GetNativeResource());
		if (!PinnedJavaCameraPlayer->GetVideoLastFrame(Resource))
		{
			return;
		}
	}

	PinnedSamples->AddVideo(VideoSample);

	FVector4 ScaleRotation = PinnedJavaCameraPlayer->GetScaleRotation();
	FVector4 Offset = PinnedJavaCameraPlayer->GetOffset();
	VideoSample->SetScaleRotationOffset(ScaleRotation, Offset);
}

struct FRHICommandUpdateTextureCameraSample final : public FRHICommand<FRHICommandUpdateTextureCameraSample>
{
	TWeakPtr<FJavaAndroidCameraPlayer, ESPMode::ThreadSafe> JavaCameraPlayerPtr;
	TWeakPtr<FMediaSamples, ESPMode::ThreadSafe> SamplesPtr;
	TSharedRef<FAndroidCameraTextureSample, ESPMode::ThreadSafe> VideoSample;
	FTimespan SampleTime;

	FORCEINLINE_DEBUGGABLE FRHICommandUpdateTextureCameraSample(TWeakPtr<FJavaAndroidCameraPlayer, ESPMode::ThreadSafe> InJavaCameraPlayerPtr, TWeakPtr<FMediaSamples, ESPMode::ThreadSafe> InSamplesPtr,
		TSharedRef<FAndroidCameraTextureSample, ESPMode::ThreadSafe> InVideoSample, FTimespan InSampleTime)
		: JavaCameraPlayerPtr(InJavaCameraPlayerPtr)
		, SamplesPtr(InSamplesPtr)
		, VideoSample(InVideoSample)
		, SampleTime(InSampleTime)
	{
	}
	void Execute(FRHICommandListBase& RHICmdList)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FRHICommandUpdateTextureCameraSample_Execute);
		DoUpdateTextureCameraSampleExecute(RHICmdList, JavaCameraPlayerPtr, SamplesPtr, VideoSample, SampleTime);
	}
};

static void DoUpdateBufferCameraSampleExecute(TWeakPtr<FJavaAndroidCameraPlayer, ESPMode::ThreadSafe> JavaCameraPlayerPtr, TWeakPtr<FMediaSamples, ESPMode::ThreadSafe> SamplesPtr,
	TSharedRef<FAndroidCameraTextureSample, ESPMode::ThreadSafe> VideoSample, int32 SampleCount, FTimespan SampleTime)
{
	auto PinnedJavaCameraPlayer = JavaCameraPlayerPtr.Pin();
	auto PinnedSamples = SamplesPtr.Pin();

	if (!PinnedJavaCameraPlayer.IsValid() || !PinnedSamples.IsValid())
	{
		return;
	}

	int32 CurrentFramePosition = 0;
	bool bRegionChanged = false;

	// write frame into buffer
	void* Buffer = nullptr;
	int64 BufferCount = 0;

	if (!PinnedJavaCameraPlayer->GetVideoLastFrameData(Buffer, BufferCount, &CurrentFramePosition, &bRegionChanged))
	{
		return;
	}

	if (BufferCount != SampleCount)
	{
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FAndroidCameraPlayer::Fetch: Sample count mismatch (Buffer=%d, Available=%d"), SampleCount, BufferCount);
	}
	check(SampleCount <= BufferCount);

	if (PinnedJavaCameraPlayer->IsActive())
	{
		// must make a copy (buffer is owned by Java, not us!)
		VideoSample->InitializeBuffer(Buffer, SampleTime, true);

		FVector4 ScaleRotation = PinnedJavaCameraPlayer->GetScaleRotation();
		FVector4 Offset = PinnedJavaCameraPlayer->GetOffset();
		VideoSample->SetScaleRotationOffset(ScaleRotation, Offset);

		PinnedSamples->AddVideo(VideoSample);
	}
}

struct FRHICommandUpdateBufferCameraSample final : public FRHICommand<FRHICommandUpdateBufferCameraSample>
{
	TWeakPtr<FJavaAndroidCameraPlayer, ESPMode::ThreadSafe> JavaCameraPlayerPtr;
	TWeakPtr<FMediaSamples, ESPMode::ThreadSafe> SamplesPtr;
	TSharedRef<FAndroidCameraTextureSample, ESPMode::ThreadSafe> VideoSample;
	int32 SampleCount;
	FTimespan SampleTime;

	FORCEINLINE_DEBUGGABLE FRHICommandUpdateBufferCameraSample(TWeakPtr<FJavaAndroidCameraPlayer, ESPMode::ThreadSafe> InJavaCameraPlayerPtr, TWeakPtr<FMediaSamples, ESPMode::ThreadSafe> InSamplesPtr,
		TSharedRef<FAndroidCameraTextureSample, ESPMode::ThreadSafe> InVideoSample, int32 InSampleCount, FTimespan InSampleTime)
		: JavaCameraPlayerPtr(InJavaCameraPlayerPtr)
		, SamplesPtr(InSamplesPtr)
		, VideoSample(InVideoSample)
		, SampleCount(InSampleCount)
		, SampleTime(InSampleTime)
	{
	}
	void Execute(FRHICommandListBase& CmdList)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FRHICommandUpdateBufferCameraSample_Execute);
		DoUpdateBufferCameraSampleExecute(JavaCameraPlayerPtr, SamplesPtr, VideoSample, SampleCount, SampleTime);
	}
};

static void DoUpdateMediaFrameDataCameraSampleExecute(TWeakPtr<FJavaAndroidCameraPlayer, ESPMode::ThreadSafe> JavaCameraPlayerPtr, TWeakPtr<FMediaSamples, ESPMode::ThreadSafe> SamplesPtr,
	TSharedRef<FAndroidCameraTextureSample, ESPMode::ThreadSafe> VideoSample, FTimespan SampleTime)
{
	auto PinnedJavaCameraPlayer = JavaCameraPlayerPtr.Pin();
	auto PinnedSamples = SamplesPtr.Pin();

	if (!PinnedJavaCameraPlayer.IsValid() || !PinnedSamples.IsValid())
	{
		return;
	}

	int32 CurrentFramePosition = 0;
	bool bRegionChanged = false;

	jobject MediaFrameData = PinnedJavaCameraPlayer->GetVideoLastMediaFrameData(CurrentFramePosition, bRegionChanged);
	if (MediaFrameData == nullptr)
	{
		return;
	}

	if (PinnedJavaCameraPlayer->IsActive())
	{
		VideoSample->InitializeMediaFrameData(MediaFrameData, SampleTime);

		FVector4 ScaleRotation = PinnedJavaCameraPlayer->GetScaleRotation();
		FVector4 Offset = PinnedJavaCameraPlayer->GetOffset();
		VideoSample->SetScaleRotationOffset(ScaleRotation, Offset);

		PinnedSamples->AddVideo(VideoSample);
	}
}

struct FRHICommandUpdateMediaFrameDataCameraSample final : public FRHICommand<FRHICommandUpdateMediaFrameDataCameraSample>
{
	TWeakPtr<FJavaAndroidCameraPlayer, ESPMode::ThreadSafe> JavaCameraPlayerPtr;
	TWeakPtr<FMediaSamples, ESPMode::ThreadSafe> SamplesPtr;
	TSharedRef<FAndroidCameraTextureSample, ESPMode::ThreadSafe> VideoSample;
	FTimespan SampleTime;

	FORCEINLINE_DEBUGGABLE FRHICommandUpdateMediaFrameDataCameraSample(TWeakPtr<FJavaAndroidCameraPlayer, ESPMode::ThreadSafe> InJavaCameraPlayerPtr, TWeakPtr<FMediaSamples, ESPMode::ThreadSafe> InSamplesPtr,
		TSharedRef<FAndroidCameraTextureSample, ESPMode::ThreadSafe> InVideoSample, FTimespan InSampleTime)
		: JavaCameraPlayerPtr(InJavaCameraPlayerPtr)
		, SamplesPtr(InSamplesPtr)
		, VideoSample(InVideoSample)
		, SampleTime(InSampleTime)
	{
	}
	void Execute(FRHICommandListBase& CmdList)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FRHICommandUpdateBufferCameraSample_Execute);
		DoUpdateMediaFrameDataCameraSampleExecute(JavaCameraPlayerPtr, SamplesPtr, VideoSample, SampleTime);
	}
};

// ==============================================================

void FAndroidCameraPlayer::TickFetch(FTimespan DeltaTime, FTimespan Timecode)
{
	if (CurrentState != EMediaState::Playing && CurrentState != EMediaState::Paused)
	{
		return;
	}

	if (!JavaCameraPlayer.IsValid() || !VideoTracks.IsValidIndex(SelectedVideoTrack))
	{
		return;
	}

	// deal with resolution changes (usually from streams)
	if (JavaCameraPlayer->DidResolutionChange())
	{
		JavaCameraPlayer->SetVideoTextureValid(false);

		// The video track dimensions need updating
		FIntPoint Dimensions = FIntPoint(JavaCameraPlayer->GetVideoWidth(), JavaCameraPlayer->GetVideoHeight());
		VideoTracks[SelectedVideoTrack].Dimensions = Dimensions;
	}

#if WITH_ENGINE

	if (FAndroidMisc::ShouldUseVulkan())
	{
		// create new video sample
		const FJavaAndroidCameraPlayer::FVideoTrack VideoTrack = VideoTracks[SelectedVideoTrack];
		auto VideoSample = VideoSamplePool->AcquireShared();

		if (!VideoSample->Initialize(VideoTrack.Dimensions, FTimespan::FromSeconds(1.0 / VideoTrack.FrameRate)))
		{
			return;
		}

		// populate & add sample (on render thread)
		const auto Settings = GetDefault<UAndroidCameraSettings>();

		struct FWriteVideoSampleParams
		{
			TWeakPtr<FJavaAndroidCameraPlayer, ESPMode::ThreadSafe> JavaCameraPlayerPtr;
			TWeakPtr<FMediaSamples, ESPMode::ThreadSafe> SamplesPtr;
			TSharedRef<FAndroidCameraTextureSample, ESPMode::ThreadSafe> VideoSample;
			int32 SampleCount;
			FTimespan SampleTime;
		}
		Params = { JavaCameraPlayer, Samples, VideoSample, (int32)(VideoTrack.Dimensions.X * VideoTrack.Dimensions.Y * sizeof(int32)), CurrentTime };

		if (JavaCameraPlayer->IsUsingBitmapRendererImageReader())
		{
			ENQUEUE_RENDER_COMMAND(AndroidCameraPlayerWriteVideoSample)(
				[Params](FRHICommandListImmediate& RHICmdList)
				{
					DoUpdateMediaFrameDataCameraSampleExecute(Params.JavaCameraPlayerPtr, Params.SamplesPtr, Params.VideoSample, Params.SampleTime);
				});
			return;
		}

		ENQUEUE_RENDER_COMMAND(AndroidCameraPlayerWriteVideoSample)(
			[Params](FRHICommandListImmediate& RHICmdList)
			{
				auto PinnedJavaCameraPlayer = Params.JavaCameraPlayerPtr.Pin();
				auto PinnedSamples = Params.SamplesPtr.Pin();

				if (!PinnedJavaCameraPlayer.IsValid() || !PinnedSamples.IsValid())
				{
					return;
				}

				int32 CurrentFramePosition = 0;
				bool bRegionChanged = false;

				// write frame into buffer
				void* Buffer = nullptr;
				int64 BufferCount = 0;

				if (!PinnedJavaCameraPlayer->GetVideoLastFrameData(Buffer, BufferCount, &CurrentFramePosition, &bRegionChanged))
				{
					return;
				}

				if (BufferCount != Params.SampleCount)
				{
					FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FAndroidCameraPlayer::Fetch: Sample count mismatch (Buffer=%d, Available=%d)"), Params.SampleCount, BufferCount);
				}
				check(Params.SampleCount <= BufferCount);

				if (PinnedJavaCameraPlayer->IsActive())
				{
					// must make a copy (buffer is owned by Java, not us!)
					Params.VideoSample->InitializeBuffer(Buffer, Params.SampleTime, true);

					FVector4 ScaleRotation = PinnedJavaCameraPlayer->GetScaleRotation();
					FVector4 Offset = PinnedJavaCameraPlayer->GetOffset();
					Params.VideoSample->SetScaleRotationOffset(ScaleRotation, Offset);

					PinnedSamples->AddVideo(Params.VideoSample);
				}
			});
	}
	else if (GSupportsImageExternal)
	{
		struct FWriteVideoSampleParams
		{
			TWeakPtr<FJavaAndroidCameraPlayer, ESPMode::ThreadSafe> JavaCameraPlayerPtr;
			FGuid PlayerGuid;
			TSharedRef<FAndroidCameraTextureSample, ESPMode::ThreadSafe> VideoSample;
			TWeakPtr<FMediaSamples, ESPMode::ThreadSafe> Samples;
			FTimespan SampleTime;
		};

		auto VideoSample = VideoSamplePool->AcquireShared();
		const FJavaAndroidCameraPlayer::FVideoTrack VideoTrack = VideoTracks[SelectedVideoTrack];
		if (!VideoSample->Initialize(VideoTrack.Dimensions, FTimespan::FromSeconds(1.0 / VideoTrack.FrameRate)))
		{
			return;
		}

		FWriteVideoSampleParams Params = { JavaCameraPlayer, PlayerGuid, VideoSample, Samples, CurrentTime };

		if (JavaCameraPlayer->IsUsingBitmapRendererImageReader())
		{

			ENQUEUE_RENDER_COMMAND(AndroidCameraPlayerWriteVideoSample)(
				[Params](FRHICommandListImmediate& RHICmdList)
				{
					if (IsRunningRHIInSeparateThread())
					{
						new (RHICmdList.AllocCommand<FRHICommandUpdateMediaFrameDataCameraSample>()) FRHICommandUpdateMediaFrameDataCameraSample(Params.JavaCameraPlayerPtr, Params.Samples, Params.VideoSample, Params.SampleTime);
						RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThread);
					}
					else
					{
						DoUpdateMediaFrameDataCameraSampleExecute(Params.JavaCameraPlayerPtr, Params.Samples, Params.VideoSample, Params.SampleTime);
					}
				});
			return;
		}

		ENQUEUE_RENDER_COMMAND(AndroidCameraPlayerWriteVideoSample)(
			[Params](FRHICommandListImmediate& RHICmdList)
			{
				ConditionalInitializeVideoTexture_RenderThread(RHICmdList, Params.JavaCameraPlayerPtr, Params.PlayerGuid);
				if (IsRunningRHIInSeparateThread())
				{
					new (RHICmdList.AllocCommand<FRHICommandUpdateExternalCameraSample>()) FRHICommandUpdateExternalCameraSample(Params.JavaCameraPlayerPtr, Params.PlayerGuid);
					RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThread);
				}
				else
				{
					DoUpdateExternalCameraSampleExecute(Params.JavaCameraPlayerPtr, Params.PlayerGuid);
				}

				auto PinnedJavaCameraPlayer = Params.JavaCameraPlayerPtr.Pin();
				auto PinnedSamples = Params.Samples.Pin();

				if (!PinnedJavaCameraPlayer.IsValid() || !PinnedSamples.IsValid())
				{
					return;
				}

				FTextureRHIRef VideoTexture = PinnedJavaCameraPlayer->GetVideoTexture();
				if (VideoTexture != nullptr && !PinnedJavaCameraPlayer->IsVideoTextureValid())
				{
					FVector4 ScaleRotation = PinnedJavaCameraPlayer->GetScaleRotation();
					FVector4 Offset = PinnedJavaCameraPlayer->GetOffset();

					#if ANDROIDCAMERAPLAYER_USE_NATIVELOGGING
						//if (bRegionChanged)
						{
							FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Fetch RT: New UV Scale/Offset = {%f, %f}, {%f, %f}  + {%f, %f} - %s"),
													ScaleRotation.X, ScaleRotation.Y, ScaleRotation.Z, ScaleRotation.W,
													Offset.X, Offset.Y, *Params.PlayerGuid.ToString());
						}
						FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Fetch RT: Register Guid: %s"), *Params.PlayerGuid.ToString());
					#endif

					FSamplerStateInitializerRHI SamplerStateInitializer(SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp);
					FSamplerStateRHIRef SamplerStateRHI = RHICreateSamplerState(SamplerStateInitializer);
					FExternalTextureRegistry::Get().RegisterExternalTexture(Params.PlayerGuid, VideoTexture, SamplerStateRHI,
																			FLinearColor(ScaleRotation.X, ScaleRotation.Y, ScaleRotation.Z, ScaleRotation.W),
																			FLinearColor(Offset.X, Offset.Y, Offset.Z, Offset.W));

					PinnedJavaCameraPlayer->SetVideoTextureValid(true);
				}

				Params.VideoSample->InitializeExternalTexture(Params.SampleTime);
				
				// Update the associated data of the registered external texture
				FTextureRHIRef SampleTexture;
				FSamplerStateRHIRef SamplerState;
				if (FExternalTextureRegistry::Get().GetExternalTexture(nullptr, Params.PlayerGuid, SampleTexture, SamplerState))
				{
					FVector4 ScaleRotation = PinnedJavaCameraPlayer->GetScaleRotation();
					FVector4 Offset = PinnedJavaCameraPlayer->GetOffset();
					FExternalTextureRegistry::Get().RegisterExternalTexture(Params.PlayerGuid, SampleTexture, SamplerState,
						FLinearColor(ScaleRotation.X, ScaleRotation.Y, ScaleRotation.Z, ScaleRotation.W),
						FLinearColor(Offset.X, Offset.Y, Offset.Z, Offset.W));
					PinnedJavaCameraPlayer->SetVideoTextureValid(true);
				}
				
				// Add a dummy video sample
				// (actual data flows "on the side" via external texture mechanism)
				PinnedSamples->AddVideo(Params.VideoSample);
			});
	}
	else
	{
		// create new video sample
		const FJavaAndroidCameraPlayer::FVideoTrack VideoTrack = VideoTracks[SelectedVideoTrack];
		auto VideoSample = VideoSamplePool->AcquireShared();

		if (!VideoSample->Initialize(VideoTrack.Dimensions, FTimespan::FromSeconds(1.0 / VideoTrack.FrameRate)))
		{
			return;
		}

		// populate & add sample (on render thread)
		//const auto Settings = GetDefault<UAndroidCameraSettings>();

		struct FWriteVideoSampleParams
		{
			TWeakPtr<FJavaAndroidCameraPlayer, ESPMode::ThreadSafe> JavaCameraPlayerPtr;
			TWeakPtr<FMediaSamples, ESPMode::ThreadSafe> SamplesPtr;
			TSharedRef<FAndroidCameraTextureSample, ESPMode::ThreadSafe> VideoSample;
			FTimespan SampleTime;
		}
		Params = { JavaCameraPlayer, Samples, VideoSample, CurrentTime };

		ENQUEUE_RENDER_COMMAND(AndroidCameraPlayerWriteVideoSample)(
			[Params](FRHICommandListImmediate& RHICmdList)
			{
				ConditionalInitializeVideoSample_RenderThread(RHICmdList, Params.VideoSample);
				if (IsRunningRHIInSeparateThread())
				{
					new (RHICmdList.AllocCommand<FRHICommandUpdateTextureCameraSample>()) FRHICommandUpdateTextureCameraSample(Params.JavaCameraPlayerPtr, Params.SamplesPtr, Params.VideoSample, Params.SampleTime);
					RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThread);
				}
				else
				{
					DoUpdateTextureCameraSampleExecute(RHICmdList, Params.JavaCameraPlayerPtr, Params.SamplesPtr, Params.VideoSample, Params.SampleTime);
				}
			});
	}

#else

	// create new video sample
	const FJavaAndroidCameraPlayer::FVideoTrack VideoTrack = VideoTracks[SelectedVideoTrack];
	auto VideoSample = VideoSamplePool->AcquireShared();

	if (!VideoSample->Initialize(VideoTrack.Dimensions, FTimespan::FromSeconds(1.0 / VideoTrack.FrameRate)))
	{
		return;
	}

	// populate & add sample (on render thread)
	const auto Settings = GetDefault<UAndroidCameraSettings>();

	struct FWriteVideoSampleParams
	{
		TWeakPtr<FJavaAndroidCameraPlayer, ESPMode::ThreadSafe> JavaCameraPlayerPtr;
		TWeakPtr<FMediaSamples, ESPMode::ThreadSafe> SamplesPtr;
		TSharedRef<FAndroidCameraTextureSample, ESPMode::ThreadSafe> VideoSample;
		int32 SampleCount;
		FTimespan SampleTime;
	}
	Params = { JavaCameraPlayer, Samples, VideoSample, (int32)(VideoTrack.Dimensions.X * VideoTrack.Dimensions.Y * sizeof(int32)), CurrentTime };

	if (JavaCameraPlayer->IsUsingBitmapRendererImageReader())
	{
		ENQUEUE_RENDER_COMMAND(AndroidCameraPlayerWriteVideoSample)(
			[Params](FRHICommandListImmediate& RHICmdList)
			{
				if (IsRunningRHIInSeparateThread())
				{
					new (RHICmdList.AllocCommand<FRHICommandUpdateMediaFrameDataCameraSample>()) FRHICommandUpdateMediaFrameDataCameraSample(Params.JavaCameraPlayerPtr, Params.SamplesPtr, Params.VideoSample, Params.SampleTime);
					RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThread);
				}
				else
				{
					DoUpdateBufferCameraSampleExecute(Params.JavaCameraPlayerPtr, Params.SamplesPtr, Params.VideoSample, Params.SampleTime);
				}
			});
		return;
	}

	ENQUEUE_RENDER_COMMAND(AndroidCameraPlayerWriteVideoSample)(
		[Params](FRHICommandListImmediate& RHICmdList)
		{
			if (IsRunningRHIInSeparateThread())
			{
				new (RHICmdList.AllocCommand<FRHICommandUpdateBufferCameraSample>()) FRHICommandUpdateBufferCameraSample(Params.JavaCameraPlayerPtr, Params.SamplesPtr, Params.VideoSample, Params.SampleCount, Params.SampleTime);
				RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThread);
			}
			else
			{
				DoUpdateBufferCameraSampleExecute(Params.JavaCameraPlayerPtr, Params.SamplesPtr, Params.VideoSample, Params.SampleCount, Params.SampleTime);
			}
		});
#endif //WITH_ENGINE
}


void FAndroidCameraPlayer::TickInput(FTimespan DeltaTime, FTimespan Timecode)
{
	if (CurrentState != EMediaState::Playing)
	{
		// remove delegates if registered
		if (ResumeHandle.IsValid() && !IsInBackground)
		{
			FCoreDelegates::ApplicationHasEnteredForegroundDelegate.Remove(ResumeHandle);
			ResumeHandle.Reset();
		}

		if (PauseHandle.IsValid())
		{
			FCoreDelegates::ApplicationWillEnterBackgroundDelegate.Remove(PauseHandle);
			PauseHandle.Reset();
		}

		if (CurrentState == EMediaState::Preparing && JavaCameraPlayer->IsPrepared())
		{
			InitializePlayer();
		}

		return;
	}

	if (!JavaCameraPlayer.IsValid())
	{
		return;
	}

	// advance time if not paused
	CurrentTime += DeltaTime * GetRate();

	// register delegate if not registered
	if (!ResumeHandle.IsValid())
	{
		ResumeHandle = FCoreDelegates::ApplicationHasEnteredForegroundDelegate.AddRaw(this, &FAndroidCameraPlayer::HandleApplicationHasEnteredForeground);
	}
	if (!PauseHandle.IsValid())
	{
		PauseHandle = FCoreDelegates::ApplicationWillEnterBackgroundDelegate.AddRaw(this, &FAndroidCameraPlayer::HandleApplicationWillEnterBackground);
	}

	// generate events
	if (!JavaCameraPlayer->IsPlaying())
	{
		if (JavaCameraPlayer->DidComplete())
		{
			EventSink.ReceiveMediaEvent(EMediaEvent::PlaybackEndReached);

			#if ANDROIDCAMERAPLAYER_USE_NATIVELOGGING
				FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FAndroidCamera::Tick - PlaybackEndReached - !playing - %s"), *PlayerGuid.ToString());
			#endif
		}

		// might catch it restarting the loop so ignore if looping
		if (!bLooping)
		{
			CurrentState = EMediaState::Stopped;
			EventSink.ReceiveMediaEvent(EMediaEvent::PlaybackSuspended);

			#if ANDROIDCAMERAPLAYER_USE_NATIVELOGGING
				FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FAndroidCamera::Tick - PlaybackSuspended - !playing - %s"), *PlayerGuid.ToString());
			#endif
		}
	}
	else if (JavaCameraPlayer->DidComplete())
	{
		EventSink.ReceiveMediaEvent(EMediaEvent::PlaybackEndReached);

		#if ANDROIDCAMERAPLAYER_USE_NATIVELOGGING
			FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FAndroidCamera::Tick - PlaybackEndReached - DidComplete true - %s"), *PlayerGuid.ToString());
		#endif
	}
}


/* FAndroidCameraPlayer implementation
 *****************************************************************************/

bool FAndroidCameraPlayer::InitializePlayer()
{
	#if ANDROIDCAMERAPLAYER_USE_NATIVELOGGING
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FAndroidCamera::InitializePlayer %s"), *PlayerGuid.ToString());
	#endif

		if (ResumeHandle.IsValid())
		{
			FCoreDelegates::ApplicationHasEnteredForegroundDelegate.Remove(ResumeHandle);
			ResumeHandle.Reset();
		}

		if (PauseHandle.IsValid())
		{
			FCoreDelegates::ApplicationWillEnterBackgroundDelegate.Remove(PauseHandle);
			PauseHandle.Reset();
		}

	JavaCameraPlayer->GetAudioTracks(AudioTracks);
	JavaCameraPlayer->GetCaptionTracks(CaptionTracks);
	JavaCameraPlayer->GetVideoTracks(VideoTracks);

	for (FJavaAndroidCameraPlayer::FVideoTrack Track : VideoTracks)
	{
		Info += FString::Printf(TEXT("Stream %i\n"), Track.Index);
		Info += TEXT("    Type: Video\n");
		Info += FString::Printf(TEXT("    MimeType: %s\n"), *Track.MimeType);
		Info += FString::Printf(TEXT("    Language: %s\n"), *Track.Language);
		Info += FString::Printf(TEXT("    Dimensions: %i x %i\n"), Track.Dimensions.X, Track.Dimensions.Y);
		Info += FString::Printf(TEXT("    FrameRate: %0.1f\n"), Track.FrameRate);
		Info += FString::Printf(TEXT("    FrameRates: %0.1f - %0.1f\n"), Track.FrameRates.GetLowerBoundValue(), Track.FrameRates.GetUpperBoundValue());
		int FormatIndex = 0;
		for (FJavaAndroidCameraPlayer::FVideoFormat Format : Track.Formats)
		{
			Info += FString::Printf(TEXT("    Format %i\n"), FormatIndex++);
			Info += FString::Printf(TEXT("        Dimensions: %i x %i\n"), Format.Dimensions.X, Format.Dimensions.Y);
			Info += FString::Printf(TEXT("        FrameRate: %0.1f\n"), Format.FrameRate);
			Info += FString::Printf(TEXT("        FrameRates: %0.1f - %0.1f\n"), Format.FrameRates.GetLowerBoundValue(), Format.FrameRates.GetUpperBoundValue());
			Info += FString::Printf(TEXT("        TypeName: BGRA\n"));
		}
		Info += TEXT("\n");
	}

	for (FJavaAndroidCameraPlayer::FAudioTrack Track : AudioTracks)
	{
		Info += FString::Printf(TEXT("Stream %i\n"), Track.Index);
		Info += TEXT("    Type: Audio\n");
//		Info += FString::Printf(TEXT("    Duration: %s\n"), *FTimespan::FromMilliseconds(Track->StreamInfo.duration).ToString());
		Info += FString::Printf(TEXT("    MimeType: %s\n"), *Track.MimeType);
		Info += FString::Printf(TEXT("    Language: %s\n"), *Track.Language);
		Info += FString::Printf(TEXT("    Channels: %i\n"), Track.Channels);
		Info += FString::Printf(TEXT("    Sample Rate: %i Hz\n"), Track.SampleRate);
		Info += TEXT("\n");
	}

	for (FJavaAndroidCameraPlayer::FCaptionTrack Track : CaptionTracks)
	{
		Info += FString::Printf(TEXT("Stream %i\n"), Track.Index);
		Info += TEXT("    Type: Caption\n");
		Info += FString::Printf(TEXT("    MimeType: %s\n"), *Track.MimeType);
		Info += FString::Printf(TEXT("    Language: %s\n"), *Track.Language);
		Info += TEXT("\n");
	}

	// Select first audio and video track by default (have to force update first time by setting to none)
	if (AudioTracks.Num() > 0)
	{
		JavaCameraPlayer->SetAudioEnabled(true);
		SelectedAudioTrack = 0;
	}
	else
	{
		JavaCameraPlayer->SetAudioEnabled(false);
		SelectedAudioTrack = INDEX_NONE;
	}

	if (VideoTracks.Num() > 0)
	{
		JavaCameraPlayer->SetVideoEnabled(true);
		SelectedVideoTrack = 0;
	}
	else
	{
		JavaCameraPlayer->SetVideoEnabled(false);
		SelectedVideoTrack = INDEX_NONE;
	}

	// set to best matching format for the selected track
	int32 NumFormats = VideoTracks[SelectedVideoTrack].Formats.Num();
	int32 NewWidth = JavaCameraPlayer->GetVideoWidth();
	int32 NewHeight = JavaCameraPlayer->GetVideoHeight();
	int32 NewFPS = JavaCameraPlayer->GetFrameRate();
	for (int32 FormatIndex = 0; FormatIndex < NumFormats; FormatIndex++)
	{
		const FJavaAndroidCameraPlayer::FVideoFormat& Format = VideoTracks[SelectedVideoTrack].Formats[FormatIndex];

		if (Format.Dimensions.X == NewWidth && Format.Dimensions.Y == NewHeight && Format.FrameRate == NewFPS)
		{
			VideoTracks[SelectedVideoTrack].Format = FormatIndex;
			break;
		}
	}

	CurrentState = EMediaState::Stopped;

	// notify listeners
	if (!bOpenWithoutEvents)
	{
		EventSink.ReceiveMediaEvent(EMediaEvent::TracksChanged);
		EventSink.ReceiveMediaEvent(EMediaEvent::MediaOpened);
	}

	return true;
}


/* IMediaTracks interface
 *****************************************************************************/

bool FAndroidCameraPlayer::GetAudioTrackFormat(int32 TrackIndex, int32 FormatIndex, FMediaAudioTrackFormat& OutFormat) const
{
	if ((FormatIndex != 0) || !AudioTracks.IsValidIndex(TrackIndex))
	{
		return false;
	}

	const FJavaAndroidCameraPlayer::FAudioTrack& Track = AudioTracks[TrackIndex];

	OutFormat.BitsPerSample = 16;
	OutFormat.NumChannels = Track.Channels;
	OutFormat.SampleRate = Track.SampleRate;
	OutFormat.TypeName = TEXT("Native");

	return true;
}


int32 FAndroidCameraPlayer::GetNumTracks(EMediaTrackType TrackType) const
{
	switch (TrackType)
	{
	case EMediaTrackType::Audio:
		return AudioTracks.Num();

	case EMediaTrackType::Caption:
		return CaptionTracks.Num();

	case EMediaTrackType::Video:
		return VideoTracks.Num();

	default:
		return 0;
	}
}


int32 FAndroidCameraPlayer::GetNumTrackFormats(EMediaTrackType TrackType, int32 TrackIndex) const
{
	if (TrackType != EMediaTrackType::Video || !VideoTracks.IsValidIndex(TrackIndex))
	{
		return 0;
	}

	return VideoTracks[TrackIndex].Formats.Num();
}


int32 FAndroidCameraPlayer::GetSelectedTrack(EMediaTrackType TrackType) const
{
	switch (TrackType)
	{
	case EMediaTrackType::Audio:
		return SelectedAudioTrack;

	case EMediaTrackType::Caption:
		return SelectedCaptionTrack;

	case EMediaTrackType::Video:
		return SelectedVideoTrack;

	default:
		return INDEX_NONE;
	}
}


FText FAndroidCameraPlayer::GetTrackDisplayName(EMediaTrackType TrackType, int32 TrackIndex) const
{
	switch (TrackType)
	{
	case EMediaTrackType::Audio:
		if (AudioTracks.IsValidIndex(TrackIndex))
		{
			return FText::FromString(AudioTracks[TrackIndex].DisplayName);
		}
		break;

	case EMediaTrackType::Caption:
		if (CaptionTracks.IsValidIndex(TrackIndex))
		{
			return FText::FromString(CaptionTracks[TrackIndex].DisplayName);
		}
		break;

	case EMediaTrackType::Video:
		if (VideoTracks.IsValidIndex(TrackIndex))
		{
			return FText::FromString(VideoTracks[TrackIndex].DisplayName);
		}

	default:
		break;
	}

	return FText::GetEmpty();
}


int32 FAndroidCameraPlayer::GetTrackFormat(EMediaTrackType TrackType, int32 TrackIndex) const
{
	if (TrackType != EMediaTrackType::Video || !VideoTracks.IsValidIndex(TrackIndex))
	{
		return INDEX_NONE;
	}

	return VideoTracks[TrackIndex].Format;
}


FString FAndroidCameraPlayer::GetTrackLanguage(EMediaTrackType TrackType, int32 TrackIndex) const
{
	switch (TrackType)
	{
	case EMediaTrackType::Audio:
		if (AudioTracks.IsValidIndex(TrackIndex))
		{
			return AudioTracks[TrackIndex].Language;
		}
		break;

	case EMediaTrackType::Caption:
		if (CaptionTracks.IsValidIndex(TrackIndex))
		{
			return CaptionTracks[TrackIndex].Language;
		}
		break;

	case EMediaTrackType::Video:
		if (VideoTracks.IsValidIndex(TrackIndex))
		{
			return VideoTracks[TrackIndex].Language;
		}

	default:
		break;
	}

	return FString();
}


FString FAndroidCameraPlayer::GetTrackName(EMediaTrackType TrackType, int32 TrackIndex) const
{
	switch (TrackType)
	{
	case EMediaTrackType::Audio:
		if (AudioTracks.IsValidIndex(TrackIndex))
		{
			return AudioTracks[TrackIndex].Name;
		}
		break;

	case EMediaTrackType::Caption:
		if (CaptionTracks.IsValidIndex(TrackIndex))
		{
			return CaptionTracks[TrackIndex].Name;
		}
		break;

	case EMediaTrackType::Video:
		if (VideoTracks.IsValidIndex(TrackIndex))
		{
			return VideoTracks[TrackIndex].Name;
		}

	default:
		break;
	}

	return FString();
}


bool FAndroidCameraPlayer::GetVideoTrackFormat(int32 TrackIndex, int32 FormatIndex, FMediaVideoTrackFormat& OutFormat) const
{
	if (!VideoTracks.IsValidIndex(TrackIndex))
	{
		return INDEX_NONE;
	}

	if (!VideoTracks[TrackIndex].Formats.IsValidIndex(FormatIndex))
	{
		return false;
	}

	const FJavaAndroidCameraPlayer::FVideoFormat& Format = VideoTracks[TrackIndex].Formats[FormatIndex];

	OutFormat.Dim = Format.Dimensions;
	OutFormat.FrameRate = Format.FrameRate;
	OutFormat.FrameRates = Format.FrameRates;
	OutFormat.TypeName = TEXT("BGRA");

	return true;
}


static FString ReplaceUrlSection(const FString SourceUrl, const FString Section, const FString Replacement)
{
	TArray<FString> UrlSections;
	int NumSections = SourceUrl.ParseIntoArray(UrlSections, TEXT("?"));

	if (NumSections < 2)
	{
		return SourceUrl + TEXT("?") + Replacement;
	}

	int SectionIndex;
	for (SectionIndex = 1; SectionIndex < NumSections; ++SectionIndex)
	{
		if (UrlSections[SectionIndex].StartsWith(Section))
		{
			UrlSections[SectionIndex] = Replacement;
			break;
		}
	}

	// reassemble
	FString NewUrl = UrlSections[0];
	for (SectionIndex = 1; SectionIndex < NumSections; ++SectionIndex)
	{
		NewUrl += "?" + UrlSections[SectionIndex];
	}

	return NewUrl;
}


bool FAndroidCameraPlayer::SelectTrack(EMediaTrackType TrackType, int32 TrackIndex)
{
	switch (TrackType)
	{
	case EMediaTrackType::Audio:
		if (TrackIndex != SelectedAudioTrack)
		{
			UE_LOGF(LogAndroidCamera, Verbose, "Player %p: Selecting audio track %i instead of %i (%i tracks)", this, TrackIndex, SelectedVideoTrack, AudioTracks.Num());

			if (TrackIndex == INDEX_NONE)
			{
				UE_LOGF(LogAndroidCamera, VeryVerbose, "Player %p: Disabling audio", this);

				JavaCameraPlayer->SetAudioEnabled(false);
			}
			else
			{
				if (!AudioTracks.IsValidIndex(TrackIndex) || !JavaCameraPlayer->SelectTrack(AudioTracks[TrackIndex].Index))
				{
					return false;
				}

				UE_LOGF(LogAndroidCamera, VeryVerbose, "Player %p: Enabling audio", this);

				JavaCameraPlayer->SetAudioEnabled(true);
			}

			SelectedAudioTrack = TrackIndex;
		}
		break;

	case EMediaTrackType::Caption:
		if (TrackIndex != SelectedCaptionTrack)
		{
			UE_LOGF(LogAndroidCamera, Verbose, "Player %p: Selecting caption track %i instead of %i (%i tracks)", this, TrackIndex, SelectedCaptionTrack, CaptionTracks.Num());

			if (TrackIndex == INDEX_NONE)
			{
				UE_LOGF(LogAndroidCamera, VeryVerbose, "Player %p: Disabling captions", this);
			}
			else
			{
				if (!CaptionTracks.IsValidIndex(TrackIndex) || JavaCameraPlayer->SelectTrack(CaptionTracks[TrackIndex].Index))
				{
					return false;
				}

				UE_LOGF(LogAndroidCamera, VeryVerbose, "Player %p: Enabling captions", this);
			}

			SelectedCaptionTrack = TrackIndex;
		}
		break;

	case EMediaTrackType::Video:
		if (TrackIndex != SelectedVideoTrack)
		{
			UE_LOGF(LogAndroidCamera, Verbose, "Player %p: Selecting video track %i instead of %i (%i tracks).", this, TrackIndex, SelectedVideoTrack, VideoTracks.Num());

			if (TrackIndex == INDEX_NONE)
			{
				UE_LOGF(LogAndroidCamera, VeryVerbose, "Player %p: Disabling video", this);
				JavaCameraPlayer->SetVideoEnabled(false);
			}
			else
			{
				if (!VideoTracks.IsValidIndex(TrackIndex))
				{
					return false;
				}

				// selecting track picks new resolution and framerate
				// restart with new framerate if open
				if (CurrentState == EMediaState::Playing || CurrentState == EMediaState::Paused || CurrentState == EMediaState::Stopped)
				{
					float OldRate = GetRate();

					int32 FormatIndex = VideoTracks[TrackIndex].Format;
					FString NewUrl = ReplaceUrlSection(MediaUrl, TEXT("width="), FString::Printf(TEXT("width=%d"), VideoTracks[TrackIndex].Formats[FormatIndex].Dimensions.X));
					NewUrl = ReplaceUrlSection(NewUrl, TEXT("height="), FString::Printf(TEXT("height=%d"), VideoTracks[TrackIndex].Formats[FormatIndex].Dimensions.Y));
					NewUrl = ReplaceUrlSection(NewUrl, TEXT("fps="), FString::Printf(TEXT("fps=%d"), (int32)VideoTracks[TrackIndex].Formats[FormatIndex].FrameRate));

					bOpenWithoutEvents = true;
					Open(NewUrl, nullptr);
					bOpenWithoutEvents = false;

					if (CurrentState != EMediaState::Stopped)
					{
						SetRate(OldRate);
					}
				}

				UE_LOGF(LogAndroidCamera, VeryVerbose, "Player %p: Enabling video", this);
				JavaCameraPlayer->SetVideoEnabled(true);
			}

			SelectedVideoTrack = TrackIndex;
		}
		break;

	default:
		return false;
	}

	return true;
}


bool FAndroidCameraPlayer::SetTrackFormat(EMediaTrackType TrackType, int32 TrackIndex, int32 FormatIndex)
{
	if (TrackType != EMediaTrackType::Video || !VideoTracks.IsValidIndex(TrackIndex))
	{
		return false;
	}

	if (!VideoTracks[TrackIndex].Formats.IsValidIndex(FormatIndex))
	{
		return false;
	}

	if (SelectedVideoTrack == TrackIndex)
	{
		if (VideoTracks[TrackIndex].Format != FormatIndex)
		{
			VideoTracks[TrackIndex].Format = FormatIndex;

			// restart with new resolution and framerate if open
			if (CurrentState == EMediaState::Playing || CurrentState == EMediaState::Paused || CurrentState == EMediaState::Stopped)
			{
				float OldRate = GetRate();

				FString NewUrl = ReplaceUrlSection(MediaUrl, TEXT("width="), FString::Printf(TEXT("width=%d"), VideoTracks[TrackIndex].Formats[FormatIndex].Dimensions.X));
				NewUrl = ReplaceUrlSection(NewUrl, TEXT("height="), FString::Printf(TEXT("height=%d"), VideoTracks[TrackIndex].Formats[FormatIndex].Dimensions.Y));
				NewUrl = ReplaceUrlSection(NewUrl, TEXT("fps="), FString::Printf(TEXT("fps=%d"), (int32)VideoTracks[TrackIndex].Formats[FormatIndex].FrameRate));

				bOpenWithoutEvents = true;
				Open(NewUrl, nullptr);
				bOpenWithoutEvents = false;

				if (CurrentState != EMediaState::Stopped)
				{
					SetRate(OldRate);
				}
			}
		}
		else
		{
			VideoTracks[TrackIndex].Format = FormatIndex;
		}
	}
	else
	{
		VideoTracks[TrackIndex].Format = FormatIndex;
	}

	return true;
}

bool FAndroidCameraPlayer::SetVideoTrackFrameRate(int32 TrackIndex, int32 FormatIndex, float FrameRate)
{
	if (CurrentState == EMediaState::Error)
	{
		return false;
	}

	if (!VideoTracks.IsValidIndex(TrackIndex))
	{
		return false;
	}

	if (!VideoTracks[TrackIndex].Formats.IsValidIndex(FormatIndex))
	{
		return false;
	}

	if (FrameRate < VideoTracks[TrackIndex].Formats[FormatIndex].FrameRates.GetLowerBoundValue() ||
		FrameRate > VideoTracks[TrackIndex].Formats[FormatIndex].FrameRates.GetUpperBoundValue())
	{
		return false;
	}

	if (SelectedVideoTrack == TrackIndex)
	{
		if (VideoTracks[TrackIndex].Format == FormatIndex)
		{
			if (VideoTracks[TrackIndex].Formats[FormatIndex].FrameRate != FrameRate)
			{
				VideoTracks[TrackIndex].Formats[FormatIndex].FrameRate = FrameRate;

				// restart with new framerate if open
				if (CurrentState == EMediaState::Playing || CurrentState == EMediaState::Paused)
				{
					float OldRate = GetRate();
					FString NewUrl = ReplaceUrlSection(MediaUrl, TEXT("fps="), FString::Printf(TEXT("fps=%d"), (int32)FrameRate));

					bOpenWithoutEvents = true;
					Open(NewUrl, nullptr);
					bOpenWithoutEvents = false;
					SetRate(OldRate);
				}
			}
		}
		else
		{
			VideoTracks[TrackIndex].Formats[FormatIndex].FrameRate = FrameRate;
		}
	}
	else
	{
		VideoTracks[TrackIndex].Formats[FormatIndex].FrameRate = FrameRate;
	}

	return true;
}

/* IMediaControls interface
 *****************************************************************************/

bool FAndroidCameraPlayer::CanControl(EMediaControl Control) const
{
	if (Control == EMediaControl::Pause)
	{
		return (CurrentState == EMediaState::Playing);
	}

	if (Control == EMediaControl::Resume)
	{
		return ((CurrentState == EMediaState::Playing) || (CurrentState == EMediaState::Stopped));
	}

	return false;
}


FTimespan FAndroidCameraPlayer::GetDuration() const
{
	if (CurrentState == EMediaState::Error)
	{
		return FTimespan::Zero();
	}

	return FTimespan::MaxValue();
}


float FAndroidCameraPlayer::GetRate() const
{
	return (CurrentState == EMediaState::Playing) ? 1.0f : 0.0f;
}


EMediaState FAndroidCameraPlayer::GetState() const
{
	return CurrentState;
}


EMediaStatus FAndroidCameraPlayer::GetStatus() const
{
	return EMediaStatus::None;
}


TRangeSet<float> FAndroidCameraPlayer::GetSupportedRates(EMediaRateThinning /*Thinning*/) const
{
	TRangeSet<float> Result;

	Result.Add(TRange<float>(0.0f));
	Result.Add(TRange<float>(1.0f));

	return Result;
}


FTimespan FAndroidCameraPlayer::GetTime() const
{
	if ((CurrentState == EMediaState::Closed) || (CurrentState == EMediaState::Error))
	{
		return FTimespan::Zero();
	}

	return CurrentTime;
}


bool FAndroidCameraPlayer::IsLooping() const
{
	return bLooping;
}


bool FAndroidCameraPlayer::Seek(const FTimespan& Time)
{
	if ((CurrentState == EMediaState::Closed) ||
		(CurrentState == EMediaState::Error) ||
		(CurrentState == EMediaState::Preparing))
	{
		UE_LOGF(LogAndroidCamera, Warning, "Cannot seek while closed, preparing, or in error state");
		return false;
	}

	UE_LOGF(LogAndroidCamera, Verbose, "Player %p: Seeking to %ls", this, *Time.ToString());

	JavaCameraPlayer->SeekTo(static_cast<int32>(Time.GetTotalMilliseconds()));
	EventSink.ReceiveMediaEvent(EMediaEvent::SeekCompleted);

	return true;
}


bool FAndroidCameraPlayer::SetLooping(bool Looping)
{
	bLooping = Looping;

	if (!JavaCameraPlayer.IsValid())
	{
		return false;
	}

	JavaCameraPlayer->SetLooping(Looping);

	return true;
}


bool FAndroidCameraPlayer::SetRate(float Rate)
{
	if ((CurrentState == EMediaState::Closed) ||
		(CurrentState == EMediaState::Error))
	{
		UE_LOGF(LogAndroidCamera, Warning, "Cannot set rate while closed, preparing, or in error state");
		return false;
	}

	if (CurrentState == EMediaState::Preparing && Rate == 1.0f)
	{
		if (IsSetRateThreadRunning)
		{
			UE_LOGF(LogAndroidCamera, Warning, "It's unnecessary to Play the camera player repeatedly.");
			return true;
		}

		UE_LOGF(LogAndroidCamera, Log, "Camera is under preparation.");

		StopSetRateThread = false;

		SetRateThreadArg* ThreadArg = new SetRateThreadArg();
		ThreadArg->ThisPtr = StaticCastSharedRef<FAndroidCameraPlayer>(AsShared());

		int ret = pthread_create(&DeferredSetRateThread, NULL, [] (void* arg) -> void* {
			SetRateThreadArg* Arg = static_cast<SetRateThreadArg*>(arg);
			if (Arg->ThisPtr.IsValid())
			{
				Arg->ThisPtr->IsSetRateThreadRunning = true;

				UE_LOGF(LogAndroidCamera, Log, "Waiting for camera to become ready...");

				float ExitTime = FPlatformTime::Seconds() + 3.0f;
				while (true)
				{
					if (FPlatformTime::Seconds() > ExitTime)
					{
						UE_LOGF(LogAndroidCamera, Log, "Timed out after waiting for camera preparation for 3 seconds.");
						break;
					}
					if (Arg->ThisPtr->StopSetRateThread)
					{
						UE_LOGF(LogAndroidCamera, Log, "Stopped waiting for camera preparation.");
						break;
					}
					if ((Arg->ThisPtr->CurrentState != EMediaState::Preparing) &&
						(Arg->ThisPtr->CurrentState != EMediaState::Error) &&
						(Arg->ThisPtr->CurrentState != EMediaState::Closed))
					{
						UE_LOGF(LogAndroidCamera, Log, "Camera is ready.");
						Arg->ThisPtr->SetRate(1.0f);
						break;
					}
					usleep(100);
				}

				Arg->ThisPtr->IsSetRateThreadRunning = false;
			}
			else
			{
				UE_LOGF(LogAndroidCamera, Error, "The camera player object has been deleted.");
			}

			delete Arg;
			return nullptr;
		}, ThreadArg);
		
		return (ret == 0);
	}

	if (Rate == GetRate())
	{
		return true; // rate already set
	}

	UE_LOGF(LogAndroidCamera, Verbose, "Player %p: Setting rate from to %f to %f", this, GetRate(), Rate);

	if (Rate == 0.0f)
	{
		if (IsSetRateThreadRunning)
		{
			StopSetRateThread = true;
			pthread_join(DeferredSetRateThread, nullptr);
			StopSetRateThread = false;
			IsSetRateThreadRunning = false;
		}

		JavaCameraPlayer->Pause();
		CurrentState = EMediaState::Paused;
		EventSink.ReceiveMediaEvent(EMediaEvent::PlaybackSuspended);
	}
	else if (Rate == 1.0f)
	{
		JavaCameraPlayer->Start();
		CurrentState = EMediaState::Playing;
		EventSink.ReceiveMediaEvent(EMediaEvent::PlaybackResumed);
	}
	else
	{
		UE_LOGF(LogAndroidCamera, Warning, "The rate %f is not supported", Rate);
		return false;
	}

	return true;
}


/* FAndroidCameraPlayer callbacks
 *****************************************************************************/

void FAndroidCameraPlayer::HandleApplicationWillEnterBackground()
{
	// prevent the HandleApplicationHasEnteredForeground from removed in TickInput()
	IsInBackground = true;

	// check state in case changed before ticked
	if (JavaCameraPlayer.IsValid())
	{
		JavaCameraPlayer->Release();
		CurrentState = EMediaState::Closed;
	}
}


void FAndroidCameraPlayer::HandleApplicationHasEnteredForeground()
{
	IsInBackground = false;

	// check state in case changed before ticked
	if (JavaCameraPlayer.IsValid() && CurrentState == EMediaState::Closed)
	{
		if (JavaCameraPlayer->RestoreCamera())
		{
			CurrentState = EMediaState::Preparing;
			SetRate(1.0f);
		}
		else
		{
			UE_LOGF(LogAndroidCamera, Warning, "HandleApplicationHasEnteredForeground - RestoreCamera failed.");
		}
	}
}


#undef LOCTEXT_NAMESPACE
