// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "CoreGlobals.h"
#include "IMediaTextureSample.h"
#include "Math/IntPoint.h"
#include "Math/Range.h"
#include "MediaObjectPool.h"
#include "Misc/Timespan.h"
#include "RHI.h"
#include "RHIUtilities.h"
#include "Templates/SharedPointer.h"
#include "PipelineStateCache.h"
#include "RHIStaticStates.h"

#if !WITH_ENGINE
	#include "MediaObjectPool.h"
#endif

#include "IMediaTextureSampleConverter.h"
#include "Android/AndroidJava.h"
#include "Android/AndroidJavaMediaFrameData.h"

/**
 * Texture sample generated by AndroidCamera player.
 */
class FAndroidCameraTextureSample
	: public IMediaTextureSample
	, public IMediaTextureSampleConverter
	, public IMediaPoolable
{
public:

	/** Default constructor. */
	FAndroidCameraTextureSample()
		: Buffer(nullptr)
		, BufferSize(0)
		, Dim(FIntPoint::ZeroValue)
		, Duration(FTimespan::Zero())
		, Time(FTimespan::Zero())
		, ScaleRotation(FLinearColor(1.0f, 0.0f, 0.0f, 1.0f))
		, Offset(FLinearColor(0.0f, 0.0f, 0.0f, 0.0f))
	{ }

	/** Virtual destructor. */
	virtual ~FAndroidCameraTextureSample()
	{
		ShutdownPoolable();
	}

public:

	/**
	 * Get a writable pointer to the sample buffer.
	 *
	 * @return Sample buffer.
	 */
	void* GetMutableBuffer()
	{
		return Buffer;
	}

	/**
	 * Initialize the sample.
	 *
	 * @param InDim The sample buffer's width and height (in pixels).
	 * @param InDuration The duration for which the sample is valid.
	 * @return true on success, false otherwise.
	 */
	bool Initialize(const FIntPoint& InDim, FTimespan InDuration)
	{
		if (InDim.GetMin() <= 0)
		{
			return false;
		}

		Dim = InDim;
		Duration = InDuration;

		return true;
	}

	/**
	 * Initialize the sample as an external image reference.
	 */
	void InitializeExternalTexture(FTimespan InTime)
	{
		Time = InTime;
		ExternalTexture = true;
	}

	/**
	 * Initialize the sample with a memory buffer.
	 *
	 * @param InBuffer The buffer containing the sample data.
	 * @param InTime The sample time (in the player's local clock).
	 * @param Copy Whether the buffer should be copied (true) or referenced (false).
	 * @see InitializeTexture
	 */
	void InitializeBuffer(void* InBuffer, FTimespan InTime, bool Copy)
	{
		Time = InTime;

		if (Copy)
		{
			const SIZE_T RequiredBufferSize = Dim.X * Dim.Y * sizeof(int32);

			if (BufferSize < RequiredBufferSize)
			{
				if (BufferSize == 0)
				{
					Buffer = FMemory::Malloc(RequiredBufferSize);
				}
				else
				{
					Buffer = FMemory::Realloc(Buffer, RequiredBufferSize);
				}

				BufferSize = RequiredBufferSize;
			}

			FMemory::Memcpy(Buffer, InBuffer, RequiredBufferSize);
		}
		else
		{
			if (BufferSize > 0)
			{
				FMemory::Free(Buffer);
				BufferSize = 0;
			}

			Buffer = InBuffer;
		}
	}

	/**
	 * Initialize the sample with a texture resource.
	 *
	* @param RHICmdList the command list used to create the texture.
	 * @param InTime The sample time (in the player's local clock).
	 * @return The texture resource object that will hold the sample data.
	 * @note This method must be called on the render thread.
	 * @see InitializeBuffer
	 */
	FRHITexture* InitializeTexture(FRHICommandListBase& RHICmdList, FTimespan InTime)
	{
		check(IsInRenderingThread() || IsInRHIThread());

		Time = InTime;

		if (Texture.IsValid() && (Texture->GetSizeXY() == Dim))
		{
			return Texture;
		}

		const FRHITextureCreateDesc Desc =
			FRHITextureCreateDesc::Create2D(TEXT("DummyTexture2D"))
			.SetExtent(Dim)
			.SetFormat(PF_B8G8R8A8)
			.SetFlags(ETextureCreateFlags::SRGB | ETextureCreateFlags::RenderTargetable | ETextureCreateFlags::ShaderResource)
			.SetInitialState(ERHIAccess::SRVMask);

		Texture = RHICmdList.CreateTexture(Desc);

		return Texture;
	}

	/**
	 * Initialize the sample with a hardware buffer extract from MediaFrameData.
	 *
	 * @param InFrameData holds the jobject of the sample data.
	 * @param InTime The sample time (in the player's local clock).
	 * @note This method must be called on the render thread.
	 * @see InitializeBuffer, InitializeTexture
	 */
	void InitializeMediaFrameData(jobject InFrameData, FTimespan InTime)
	{
		MediaFrameData.CleanUp();
		MediaFrameData.Set(InFrameData);
		Time = InTime;
	}

	/**
	 * Set the sample Scale, Rotation, Offset.
	 *
	 * @param InScaleRotation The sample scale and rotation transform (2x2).
	 * @param InOffset The sample offset.
	 */
	void SetScaleRotationOffset(FVector4& InScaleRotation, FVector4& InOffset)
	{
		ScaleRotation = FLinearColor(InScaleRotation.X, InScaleRotation.Y, InScaleRotation.Z, InScaleRotation.W);
		Offset = FLinearColor(InOffset.X, InOffset.Y, InOffset.Z, InOffset.W);
	}

public:

	//~ IMediaTextureSample interface

	virtual const void* GetBuffer() override
	{
		return Buffer;
	}

	virtual FIntPoint GetDim() const override
	{
		return Dim;
	}

	virtual FTimespan GetDuration() const override
	{
		return Duration;
	}

	virtual EMediaTextureSampleFormat GetFormat() const override
	{
		return EMediaTextureSampleFormat::CharBGRA;
	}

	virtual FIntPoint GetOutputDim() const override
	{
		return Dim;
	}

	virtual uint32 GetStride() const override
	{
		return Dim.X * sizeof(int32);
	}

#if WITH_ENGINE

	virtual FRHITexture* GetTexture() const override
	{
		return Texture.GetReference();
	}

#endif //WITH_ENGINE

	virtual FMediaTimeStamp GetTime() const override
	{
		return FMediaTimeStamp(Time);
	}

	virtual bool IsExternalImage() const override
	{
		return ExternalTexture;
	}

	virtual bool IsCacheable() const override
	{
#if WITH_ENGINE
		return true;
#else
		return (BufferSize > 0);
#endif
	}

	virtual bool IsOutputSrgb() const override
	{
		return true;
	}

	virtual FLinearColor GetScaleRotation() const override
	{
		return ScaleRotation;
	}

	virtual FLinearColor GetOffset() const override
	{
		return Offset;
	}

	virtual EMediaOrientation GetOrientation() const override
	{
		return EMediaOrientation::CW90;
	}

	//~ IMediaTextureSampleConverter interface
	virtual IMediaTextureSampleConverter* GetMediaTextureSampleConverter() override
	{
		if (MediaFrameData || Buffer)
		{
			return this;
		}

		return nullptr;
	}

	//~ IMediaPoolable interface
	virtual bool IsReadyForReuse() override
	{
		if (MediaFrameData)
		{
			return MediaFrameData.IsReadyToClean();
		}
		return true;
	}

	virtual void ShutdownPoolable() override
	{
		// Reuse the texture?
		//Texture = nullptr;

		if (BufferSize > 0)
		{
			FMemory::Free(Buffer);
			BufferSize = 0;
			Buffer = nullptr;
		}

		MediaFrameData.CleanUp();
	}

private:

	bool RotateBinaryBufferTexture(FRHICommandListImmediate& RHICmdList, FTextureRHIRef& InDstTexture, const FConversionHints& Hints)
	{
		const FRHITextureCreateDesc Desc =
			FRHITextureCreateDesc::Create2D(TEXT("CameraSampleConverterIntermediateTexture"))
			.SetExtent(Dim)
			.SetFormat(PF_B8G8R8A8)
			.SetNumMips(Hints.NumMips)
			.DetermineInititialState();
		FTextureRHIRef IntermediateTexture = RHICmdList.CreateTexture(Desc);

		if (!IntermediateTexture.IsValid())
		{
			return false;
		}

		// Copy sample data (from CPU mem) to output render target
		const FPixelFormatInfo& FormatInfo = GPixelFormats[PF_B8G8R8A8];
		const uint8* Data = (const uint8*)Buffer;
		for (uint8 MipLevel = 0; MipLevel < Hints.NumMips; ++MipLevel)
		{
			uint32 Width = Dim.X >> MipLevel;
			uint32 Height = Dim.Y >> MipLevel;
			uint32 Stride = GetStride() >> MipLevel;

			FUpdateTextureRegion2D Region(0, 0, 0, 0, Width, Height);
			RHICmdList.UpdateTexture2D(IntermediateTexture, MipLevel, Region, Stride, Data);

			uint32 HeightUnit = (Height + FormatInfo.BlockSizeY - 1) / FormatInfo.BlockSizeY;
			Data += Stride * HeightUnit;
		}

		FGraphicsPipelineStateInitializer GraphicsPSOInit;
		FRHITexture* RenderTarget = InDstTexture.GetReference();

		RHICmdList.Transition(FRHITransitionInfo(InDstTexture, ERHIAccess::Unknown, ERHIAccess::RTV));

		FRHIRenderPassInfo RPInfo(RenderTarget, ERenderTargetActions::DontLoad_Store);
		RHICmdList.BeginRenderPass(RPInfo, TEXT("ConvertMedia_BinaryBuffer"));
		{
			RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);

			RHICmdList.SetViewport(0, 0, 0.0f, (float)Dim.X, (float)Dim.Y, 1.0f);

			GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
			GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
			GraphicsPSOInit.BlendState = TStaticBlendStateWriteMask<CW_RGBA, CW_NONE, CW_NONE, CW_NONE, CW_NONE, CW_NONE, CW_NONE, CW_NONE>::GetRHI();
			GraphicsPSOInit.PrimitiveType = PT_TriangleStrip;

			// configure media shaders
			auto ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
			TShaderMapRef<FMediaShadersVS> VertexShader(ShaderMap);

			GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GMediaVertexDeclaration.VertexDeclarationRHI;
			GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();

			FSamplerStateInitializerRHI InSamplerStateInitializer(SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp);
			FSamplerStateRHIRef InSamplerState = RHICreateSamplerState(InSamplerStateInitializer);

			TShaderMapRef<FReadTextureExternalPS> CopyShader(ShaderMap);
			GraphicsPSOInit.BoundShaderState.PixelShaderRHI = CopyShader.GetPixelShader();
			SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);
			SetShaderParametersLegacyPS(RHICmdList, CopyShader, IntermediateTexture, InSamplerState, ScaleRotation, Offset);

			// draw full size quad into render target
			FBufferRHIRef VertexBuffer = CreateTempMediaVertexBuffer(RHICmdList);
			RHICmdList.SetStreamSource(0, VertexBuffer, 0);
			// set viewport to RT size
			RHICmdList.SetViewport(0, 0, 0.0f, (float)Dim.X, (float)Dim.Y, 1.0f);

			RHICmdList.DrawPrimitive(0, 2, 1);
		}
		RHICmdList.EndRenderPass();

		return true;
	}

	//~ IMediaTextureSampleConverter overrides
	virtual bool Convert(FRHICommandListImmediate& RHICmdList, FTextureRHIRef& InDstTexture, const FConversionHints& Hints) override
	{
		if (MediaFrameData)
		{
			if (FAndroidMisc::ShouldUseVulkan())
			{
				return MediaFrameData.ExtractToTextureVulkan(RHICmdList, InDstTexture, this);
			}
			else
			{
				return MediaFrameData.ExtractToTextureOES(RHICmdList, InDstTexture, this);
			}
		}

		if (Buffer != nullptr && BufferSize > 0)
		{
			if (FAndroidMisc::ShouldUseVulkan())
			{
				return RotateBinaryBufferTexture(RHICmdList, InDstTexture, Hints);
			}
		}

		return false;
	}

	/** The sample's frame data from the java surface provider. */
	FAndroidJavaMediaFrameData MediaFrameData;

	/** The sample's data buffer. */
	void* Buffer;

	/** Current allocation size of Buffer. */
	SIZE_T BufferSize;

	/** Flag indicating if this is an external image reference. */
	bool ExternalTexture;

	/** Width and height of the texture sample. */
	FIntPoint Dim;

	/** Duration for which the sample is valid. */
	FTimespan Duration;

	/** Sample time. */
	FTimespan Time;

	/** ScaleRotation for the sample. */
	FLinearColor ScaleRotation;

	/** Offset for the sample. */
	FLinearColor Offset;

#if WITH_ENGINE

	/** Texture resource. */
	TRefCountPtr<FRHITexture> Texture;

#endif //WITH_ENGINE
};


/** Implements a pool for Android texture sample objects. */
class FAndroidCameraTextureSamplePool : public TMediaObjectPool<FAndroidCameraTextureSample> { };
