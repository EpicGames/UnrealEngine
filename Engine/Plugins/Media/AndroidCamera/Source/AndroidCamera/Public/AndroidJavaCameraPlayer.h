// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Android/AndroidJava.h"
#include "RHI.h"
#include "RHIResources.h"

// Wrapper for com/epicgames/unreal/CameraPlayer*.java.
class FJavaAndroidCameraPlayer : public FJavaClassObject
{
public:
	enum FPlayerState
	{
		Inactive,
		Active
	};

	struct FAudioTrack
	{
		int32 Index;
		FString MimeType;
		FString DisplayName;
		FString Language;
		FString Name;
		uint32 Channels;
		uint32 SampleRate;
	};

	struct FCaptionTrack
	{
		int32 Index;
		FString MimeType;
		FString DisplayName;
		FString Language;
		FString Name;
	};

	struct FVideoFormat
	{
		FIntPoint Dimensions;
		float FrameRate;
		TRange<float> FrameRates;
	};

	struct FVideoTrack
	{
		int32 Index;
		FString MimeType;
		FString DisplayName;
		FString Language;
		FString Name;
		uint32 BitRate;
		FIntPoint Dimensions;
		float FrameRate;
		TRange<float> FrameRates;
		int32 Format;
		TArray<FVideoFormat> Formats;
	};

public:
	FJavaAndroidCameraPlayer(bool swizzlePixels, bool vulkanRenderer);
	virtual ~FJavaAndroidCameraPlayer();
	int32 GetDuration();
	bool IsActive();
	void Reset();
	void Stop();
	int32 GetCurrentPosition();
	bool IsLooping();
	bool IsPlaying();
	bool IsPrepared();
	bool DidComplete();
	bool SetDataSource(const FString & Url);
	FString GetDataSource();
	bool Prepare();
	bool PrepareAsync();
	void SeekTo(int32 Milliseconds);
	void SetLooping(bool Looping);
	bool RestoreCamera();
	void Release();
	int32 GetVideoHeight();
	int32 GetVideoWidth();
	float GetFrameRate();
	void SetVideoEnabled(bool enabled = true);
	void SetAudioEnabled(bool enabled = true);
	bool IsUsingBitmapRendererImageReader();
	jobject GetVideoLastMediaFrameData(int32& CurrentPosition, bool& bRegionChanged);
	bool GetVideoLastFrameData(void* & outPixels, int64 & outCount, int32 *CurrentPosition, bool *bRegionChanged);
	void Start();
	void Pause();
	bool GetVideoLastFrame(int32 destTexture);
	bool SelectTrack(int32 index);
	bool GetAudioTracks(TArray<FAudioTrack>& AudioTracks);
	bool GetCaptionTracks(TArray<FCaptionTrack>& CaptionTracks);
	bool GetVideoTracks(TArray<FVideoTrack>& VideoTracks);
	bool DidResolutionChange();
	int32 GetExternalTextureId();
	bool UpdateVideoFrame(int32 ExternalTextureId, int32 *CurrentPosition, bool *bRegionChanged);
	bool TakePicture(const FString& Filename);
	bool TakePicture(const FString& Filename, int32 Width, int32 Height);

private:
	static FName GetClassName();

	FPlayerState PlayerState;

	bool bTrackInfoSupported;

	FJavaClassMethod GetDurationMethod;
	FJavaClassMethod ResetMethod;
	FJavaClassMethod StopMethod;
	FJavaClassMethod GetCurrentPositionMethod;
	FJavaClassMethod IsLoopingMethod;
	FJavaClassMethod IsPlayingMethod;
	FJavaClassMethod IsPreparedMethod;
	FJavaClassMethod DidCompleteMethod;
	FJavaClassMethod SetDataSourceURLMethod;
	FJavaClassMethod GetDataSourceURLMethod;
	FJavaClassMethod PrepareMethod;
	FJavaClassMethod PrepareAsyncMethod;
	FJavaClassMethod SeekToMethod;
	FJavaClassMethod SetLoopingMethod;
	FJavaClassMethod OpenCameraMethod;
	FJavaClassMethod ReleaseMethod;
	FJavaClassMethod GetVideoHeightMethod;
	FJavaClassMethod GetVideoWidthMethod;
	FJavaClassMethod GetFrameRateMethod;
	FJavaClassMethod SetVideoEnabledMethod;
	FJavaClassMethod SetAudioEnabledMethod;
	FJavaClassMethod GetVideoLastFrameDataMethod;
	FJavaClassMethod StartMethod;
	FJavaClassMethod PauseMethod;
	FJavaClassMethod GetVideoLastFrameMethod;
	FJavaClassMethod SelectTrackMethod;
	FJavaClassMethod GetAudioTracksMethod;
	FJavaClassMethod GetCaptionTracksMethod;
	FJavaClassMethod GetVideoTracksMethod;
	FJavaClassMethod DidResolutionChangeMethod;
	FJavaClassMethod GetExternalTextureIdMethod;
	FJavaClassMethod UpdateVideoFrameMethod;
	FJavaClassMethod TakePictureMethod;

	// interfaces of getting video frame data through ImageReader
	FJavaClassMethod SetUsingBitmapRendererImageReaderMethod;
	FJavaClassMethod GetVideoLastMediaFrameDataMethod;

	// FrameUpdateInfo member field ids
	jclass FrameUpdateInfoClass;
	jfieldID FrameUpdateInfo_Buffer;
	jfieldID FrameUpdateInfo_CurrentPosition;
	jfieldID FrameUpdateInfo_FrameReady;
	jfieldID FrameUpdateInfo_RegionChanged;
	jfieldID FrameUpdateInfo_ScaleRotation00;
	jfieldID FrameUpdateInfo_ScaleRotation01;
	jfieldID FrameUpdateInfo_ScaleRotation10;
	jfieldID FrameUpdateInfo_ScaleRotation11;
	jfieldID FrameUpdateInfo_UOffset;
	jfieldID FrameUpdateInfo_VOffset;
	jfieldID FrameUpdateInfo_BRFrameData;

	// AudioDeviceInfo member field ids
	jclass AudioTrackInfoClass;
	jfieldID AudioTrackInfo_Index;
	jfieldID AudioTrackInfo_MimeType;
	jfieldID AudioTrackInfo_DisplayName;
	jfieldID AudioTrackInfo_Language;
	jfieldID AudioTrackInfo_Channels;
	jfieldID AudioTrackInfo_SampleRate;

	// AudioDeviceInfo member field ids
	jclass CaptionTrackInfoClass;
	jfieldID CaptionTrackInfo_Index;
	jfieldID CaptionTrackInfo_MimeType;
	jfieldID CaptionTrackInfo_DisplayName;
	jfieldID CaptionTrackInfo_Language;

	// AudioDeviceInfo member field ids
	jclass VideoTrackInfoClass;
	jfieldID VideoTrackInfo_Index;
	jfieldID VideoTrackInfo_MimeType;
	jfieldID VideoTrackInfo_DisplayName;
	jfieldID VideoTrackInfo_Language;
	jfieldID VideoTrackInfo_BitRate;
	jfieldID VideoTrackInfo_Width;
	jfieldID VideoTrackInfo_Height;
	jfieldID VideoTrackInfo_FrameRate;
	jfieldID VideoTrackInfo_FrameRateLow;
	jfieldID VideoTrackInfo_FrameRateHigh;

	FTextureRHIRef VideoTexture;
	bool bVideoTextureValid;

	FVector4 ScaleRotation;
	FVector4 Offset;

	enum BitmapRendererSelection
	{
		Unknown,
		Legacy,
		ImageReader
	};

	BitmapRendererSelection BRSelection; // BR - BitmapRenderer
	bool UsingVulkanRenderer;

	void SelectBitmapRenderer();
	void UpdateScaleRotationOffset();

public:
	FTextureRHIRef GetVideoTexture()
	{
		return VideoTexture;
	}

	void SetVideoTexture(FTextureRHIRef Texture)
	{
		VideoTexture = Texture;
	}

	void SetVideoTextureValid(bool Condition)
	{
		bVideoTextureValid = Condition;
	}

	bool IsVideoTextureValid()
	{
		return bVideoTextureValid;
	}

	FVector4 GetScaleRotation() { return ScaleRotation; }

	FVector4 GetOffset() { return Offset; }
};
