// Copyright Epic Games, Inc. All Rights Reserved.

#include "AndroidCameraEditor.h"
#include "ISettingsModule.h"
#include "AndroidCameraRuntimeSettings.h"
#include "Modules/ModuleManager.h"

/**
 * Implements the AndroidCameraEditor module.
 */

#define LOCTEXT_NAMESPACE "AndroidCamera"

void FAndroidCameraEditorModule::StartupModule()
{
	// register settings
	ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

	if (SettingsModule != nullptr)	
	{
		SettingsModule->RegisterSettings("Project", "Plugins", "AndroidCamera",
			LOCTEXT("CameraSettingsName", "Android Camera"),
			LOCTEXT("CameraSettingsDescription", "Project settings for Android camera plugin"),
			GetMutableDefault<UAndroidCameraRuntimeSettings>()
		);
	}
}

void FAndroidCameraEditorModule::ShutdownModule()
{
	// unregister settings
	ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

	if (SettingsModule != nullptr)
	{
       		SettingsModule->UnregisterSettings("Project", "Plugins", "AndroidCamera");
	}
}


IMPLEMENT_MODULE(FAndroidCameraEditorModule, AndroidCameraEditor);

#undef LOCTEXT_NAMESPACE
