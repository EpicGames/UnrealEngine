// Copyright Epic Games, Inc. All Rights Reserved.

#include "AndroidCameraRuntimeSettings.h"

//////////////////////////////////////////////////////////////////////////
// UAndroidCameraRuntimeSettings

UAndroidCameraRuntimeSettings::UAndroidCameraRuntimeSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bEnablePermission(true)
	, bRequiresAnyCamera(false)
	, bRequiresBackFacingCamera(false)
	, bRequiresFrontFacingCamera(false)
{
}
