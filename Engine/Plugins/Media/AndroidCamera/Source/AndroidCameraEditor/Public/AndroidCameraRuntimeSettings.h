// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Engine/EngineTypes.h"
#include "AndroidCameraRuntimeSettings.generated.h"

/**
* Implements the settings for the AndroidCamera plugin.
*/
UCLASS(Config = Engine, DefaultConfig)
class UAndroidCameraRuntimeSettings : public UObject
{
	GENERATED_UCLASS_BODY()

	// Enable camera permission in AndroidManifest
	UPROPERTY(EditAnywhere, config, Category = ManifestSettings)
	bool bEnablePermission;

	// Requires a camera to operate (if true and back-facing and front-facing are false, sets android.hardware.camera.any as required)
	UPROPERTY(EditAnywhere, config, Category = ManifestSettings)
	bool bRequiresAnyCamera;

	// Requires back-facing camera in AndroidManifest (android.hardware.camera)
	UPROPERTY(EditAnywhere, config, Category = ManifestSettings)
	bool bRequiresBackFacingCamera;

	// Requires front-facing camera in AndroidManifest (android.hardware.camera.front)
	UPROPERTY(EditAnywhere, config, Category = ManifestSettings)
	bool bRequiresFrontFacingCamera;
};
