// Copyright Epic Games, Inc. All Rights Reserved.

#include "AndroidCameraFactoryPrivate.h"

#include "IAndroidCameraModule.h"
#include "Internationalization/Internationalization.h"
#include "Misc/Guid.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "IMediaModule.h"
#include "IMediaPlayerFactory.h"

DEFINE_LOG_CATEGORY(LogAndroidCameraFactory);

#define LOCTEXT_NAMESPACE "FAndroidCameraFactoryModule"


/**
 * Implements the AndroidCameraFactory module.
 */
class FAndroidCameraFactoryModule
	: public IMediaPlayerFactory
	, public IModuleInterface
{
public:

	//~ IMediaPlayerFactory interface

	virtual bool CanPlayUrl(const FString& Url, const IMediaOptions* Options, TArray<FText>* OutWarnings, TArray<FText>* OutErrors) const override
	{
		return GetPlayabilityConfidenceScore(Url, Options, OutWarnings, OutErrors) > 0 ? true : false;
	}

	virtual int32 GetPlayabilityConfidenceScore(const FString& Url, const IMediaOptions* Options, TArray<FText>* OutWarnings, TArray<FText>* OutErrors) const override
	{
		FString Scheme;
		FString Location;

		// check scheme
		if (!Url.Split(TEXT("://"), &Scheme, &Location, ESearchCase::CaseSensitive))
		{
			if (OutErrors != nullptr)
			{
				OutErrors->Add(LOCTEXT("NoSchemeFound", "No URI scheme found"));
			}

			return 0;
		}

		if (!SupportedUriSchemes.Contains(Scheme))
		{
			if (OutErrors != nullptr)
			{
				OutErrors->Add(FText::Format(LOCTEXT("SchemeNotSupported", "The URI scheme '{0}' is not supported"), FText::FromString(Scheme)));
			}

			return 0;
		}

		// check file extension
		if (Scheme == TEXT("file"))
		{
			const FString Extension = FPaths::GetExtension(Location, false);

			if (!SupportedFileExtensions.Contains(Extension))
			{
				if (OutErrors != nullptr)
				{
					OutErrors->Add(FText::Format(LOCTEXT("ExtensionNotSupported", "The file extension '{0}' is not supported"), FText::FromString(Extension)));
				}

				return 0;
			}
		}

		return 100;
	}

	virtual TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> CreatePlayer(IMediaEventSink& EventSink) override
	{
		auto AndroidCameraModule = FModuleManager::LoadModulePtr<IAndroidCameraModule>("AndroidCamera");
		return (AndroidCameraModule != nullptr) ? AndroidCameraModule->CreatePlayer(EventSink) : nullptr;
	}

	virtual FText GetDisplayName() const override
	{
		return LOCTEXT("MediaPlayerDisplayName", "Android Camera");
	}

	virtual FName GetPlayerName() const override
	{
		static FName PlayerName(TEXT("AndroidCamera"));
		return PlayerName;
	}

	virtual FGuid GetPlayerPluginGUID() const override
	{
		static FGuid PlayerPluginGUID(0x94f59418, 0xb93d4f40, 0xa441b4fd, 0x2ac7c313);
		return PlayerPluginGUID;
	}

	virtual const TArray<FString>& GetSupportedPlatforms() const override
	{
		return SupportedPlatforms;
	}

	virtual bool SupportsFeature(EMediaFeature Feature) const override
	{
		return ((Feature == EMediaFeature::VideoSamples) ||
				(Feature == EMediaFeature::VideoTracks));
	}

public:

	//~ IModuleInterface interface

	virtual void StartupModule() override
	{
		// supported file extensions

		// supported platforms
		SupportedPlatforms.Add(TEXT("Android"));

		// supported schemes
		SupportedUriSchemes.Add(TEXT("vidcap"));

		// register media player info
		auto MediaModule = FModuleManager::LoadModulePtr<IMediaModule>("Media");

		if (MediaModule != nullptr)
		{
			MediaModule->RegisterPlayerFactory(*this);
		}
	}

	virtual void ShutdownModule() override
	{
		// unregister player factory
		auto MediaModule = FModuleManager::GetModulePtr<IMediaModule>("Media");

		if (MediaModule != nullptr)
		{
			MediaModule->UnregisterPlayerFactory(*this);
		}
	}

private:

	/** List of supported media file types. */
	TArray<FString> SupportedFileExtensions;

	/** List of platforms that the media player support. */
	TArray<FString> SupportedPlatforms;

	/** List of supported URI schemes. */
	TArray<FString> SupportedUriSchemes;
};


#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FAndroidCameraFactoryModule, AndroidCameraFactory);
