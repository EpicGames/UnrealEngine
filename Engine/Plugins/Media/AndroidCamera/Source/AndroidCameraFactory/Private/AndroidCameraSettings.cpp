// Copyright Epic Games, Inc. All Rights Reserved.

#include "AndroidCameraSettings.h"


/* UAndroidMediaSettings structors
 *****************************************************************************/

UAndroidCameraSettings::UAndroidCameraSettings()
	: CacheableVideoSampleBuffers(false)
{ }
