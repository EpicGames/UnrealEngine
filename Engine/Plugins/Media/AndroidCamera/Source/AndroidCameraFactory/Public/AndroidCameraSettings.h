// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "AndroidCameraSettings.generated.h"


/**
 * Settings for the ImgMedia module.
 */
UCLASS(config=Engine)
class ANDROIDCAMERAFACTORY_API UAndroidCameraSettings
	: public UObject
{
	GENERATED_BODY()

public:

	/** 
	 * Whether video samples should be cacheable (default = off).
	 *
	 * This setting only affects applications that are not compiled against the
	 * Engine. In such applications, the video samples transfer their contents
	 * via a frame buffer. By default, the same frame buffer is reused for every
	 * sample to avoid buffer copies. Every time a new sample is generated, the
	 * previously generated samples are invalidated.
	 *
	 * When enabling this option, video frame buffers are copied into instead of
	 * referenced in video samples. This may be useful for applications that require
	 * access to individual frames, but it may dramatically decrease performance.
	 *
	 * When compiling against the Engine, this setting has no effect as the frame
	 * data is transferred via separate texture resource objects.
	 */
	UPROPERTY(config, EditAnywhere, Category="Video")
	bool CacheableVideoSampleBuffers;

public:

	/** Default constructor. */
	UAndroidCameraSettings();
};
